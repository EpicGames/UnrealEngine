{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "ML Deformer Neural Morph Model",
	"Description": "Neural Morph Model for the ML Deformer Framework",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://docs.unrealengine.com/5.0/en-US/using-the-machine-learning-deformer-in-unreal-engine/",
	"MarketplaceURL": "",
	"SupportURL": "https://forums.unrealengine.com/",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "NeuralMorphModel",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "NeuralMorphModelEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "GeometryCache",
			"Enabled": true
		},
		{
			"Name": "DeformerGraph",
			"Enabled": true
		},
		{
			"Name": "PythonMLPackages",
			"Enabled": true
		},
		{
			"Name": "MLDeformerFramework",
			"Enabled": true
		},
		{
			"Name": "NNERuntimeBasicCpu",
			"Enabled": true
		}
	]
}
