// Copyright Epic Games, Inc. All Rights Reserved.

#include "NeuralMorphInputInfo.h"
#include "NeuralMorphTypes.h"
#include "MLDeformerCurveReference.h"
#include "BoneContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NeuralMorphInputInfo)

void UNeuralMorphInputInfo::Reset()
{
	Super::Reset();
	BoneGroups.Empty();
	CurveGroups.Empty();
}

TArray<FNeuralMorphBoneGroup>& UNeuralMorphInputInfo::GetBoneGroups()
{
	return BoneGroups;
}

const TArray<FNeuralMorphBoneGroup>& UNeuralMorphInputInfo::GetBoneGroups() const
{
	return BoneGroups;
}

TArray<FNeuralMorphCurveGroup>& UNeuralMorphInputInfo::GetCurveGroups()
{
	return CurveGroups;
}

const TArray<FNeuralMorphCurveGroup>& UNeuralMorphInputInfo::GetCurveGroups() const
{
	return CurveGroups;
}

int32 UNeuralMorphInputInfo::CalcMaxNumGroupItems() const
{
	int32 MaxGroupSize = 0;

	const int32 NumBoneGroups = BoneGroups.Num();
	for (int32 GroupIndex = 0; GroupIndex < NumBoneGroups; ++GroupIndex)
	{
		const int32 NumItemsInGroup = BoneGroups[GroupIndex].BoneNames.Num();
		MaxGroupSize = FMath::Max<int32>(MaxGroupSize, NumItemsInGroup);
	}

	const int32 NumCurveGroups = CurveGroups.Num();
	for (int32 GroupIndex = 0; GroupIndex < NumCurveGroups; ++GroupIndex)
	{
		const int32 NumItemsInGroup = CurveGroups[GroupIndex].CurveNames.Num();
		MaxGroupSize = FMath::Max<int32>(MaxGroupSize, NumItemsInGroup);
	}

	return MaxGroupSize;
}

void UNeuralMorphInputInfo::GenerateBoneGroupIndices(TArray<int32>& OutBoneGroupIndices)
{
	const int32 MaxGroupSize = CalcMaxNumGroupItems();

	// Make sure we have the right number of groups.
	const int32 NumBoneGroups = BoneGroups.Num();
	if (OutBoneGroupIndices.Num() != NumBoneGroups * MaxGroupSize)
	{
		OutBoneGroupIndices.Reset();
		OutBoneGroupIndices.AddDefaulted(NumBoneGroups * MaxGroupSize);
	}

	// Build a list of ints for every group.
	// The integers in the group are the bone indices we pass transforms for to the inputs.
	// 0 means the first bone we pass to the network inputs.
	// 9 means the tenth bone we pass to the network inputs, etc.
	// All groups will have the same number of items. We pad them with INDEX_NONE when they have less items
	// compared to the maximum number of items of all groups.
	for (int32 BoneGroupIndex = 0; BoneGroupIndex < NumBoneGroups; ++BoneGroupIndex)
	{
		int32 LastValidIndex = INDEX_NONE;
		const int32 NumItemsInGroup = BoneGroups[BoneGroupIndex].BoneNames.Num();
		for (int32 Index = 0; Index < MaxGroupSize; Index++)
		{
			if (Index < NumItemsInGroup)
			{
				const FName BoneName = BoneGroups[BoneGroupIndex].BoneNames[Index].BoneName;
				const int32 BoneInputIndex = BoneName.IsValid() ? BoneNames.Find(BoneName) : INDEX_NONE;
				if (BoneInputIndex != INDEX_NONE)
				{
					LastValidIndex = BoneInputIndex;
				}
				OutBoneGroupIndices[BoneGroupIndex * MaxGroupSize + Index] = BoneInputIndex;
			}
			else
			{
				OutBoneGroupIndices[BoneGroupIndex * MaxGroupSize + Index] = LastValidIndex;
			}
		}
	}
}

void UNeuralMorphInputInfo::GenerateCurveGroupIndices(TArray<int32>& OutCurveGroupIndices)
{
	const int32 MaxGroupSize = CalcMaxNumGroupItems();

	// Make sure we have the right number of groups.
	const int32 NumCurveGroups = CurveGroups.Num();
	if (OutCurveGroupIndices.Num() != NumCurveGroups * MaxGroupSize)
	{
		OutCurveGroupIndices.Reset();
		OutCurveGroupIndices.AddDefaulted(NumCurveGroups * MaxGroupSize);
	}

	// Build a list of ints for every group.
	// The integers in the group are the curve indices, for curve values we pass to the network as inputs.
	// 0 means the first curve we pass to the network inputs.
	// 9 means the tenth curve we pass to the network inputs, etc.
	// All groups will have the same number of items. We pad them with INDEX_NONE when they have less items
	// compared to the maximum number of items of all groups.
	for (int32 CurveGroupIndex = 0; CurveGroupIndex < NumCurveGroups; ++CurveGroupIndex)
	{
		int32 LastValidIndex = INDEX_NONE;
		const int32 NumItemsInGroup = CurveGroups[CurveGroupIndex].CurveNames.Num();
		for (int32 Index = 0; Index < MaxGroupSize; Index++)
		{
			if (Index < NumItemsInGroup)
			{
				const FName CurveName = CurveGroups[CurveGroupIndex].CurveNames[Index].CurveName;
				int32 CurveInputIndex = CurveName.IsValid() ? CurveNames.Find(CurveName) : INDEX_NONE;
				if (CurveInputIndex != INDEX_NONE)
				{
					LastValidIndex = CurveInputIndex;
				}
				OutCurveGroupIndices[CurveGroupIndex * MaxGroupSize + Index] = CurveInputIndex;
			}
			else
			{
				OutCurveGroupIndices[CurveGroupIndex * MaxGroupSize + Index] = LastValidIndex;
			}
		}
	}
}

bool UNeuralMorphInputInfo::HasInvalidGroups() const
{
	// Check all bone groups.
	for (const FNeuralMorphBoneGroup& BoneGroup : BoneGroups)
	{
		if (BoneGroup.BoneNames.Num() <= 1)
		{
			return true;
		}

		// We require all bone names to be valid.
		const int32 NumCurves = BoneGroup.BoneNames.Num();
		for (int32 Index = 0; Index < NumCurves; ++Index)
		{
			const FName Name = BoneGroup.BoneNames[Index].BoneName;
			if (!Name.IsValid() || Name.IsNone())
			{
				return true;
			}

			if (!BoneNames.Contains(Name))
			{
				return true;
			}
		}
	}

	// Check all curve groups.
	for (const FNeuralMorphCurveGroup& CurveGroup : CurveGroups)
	{
		if (CurveGroup.CurveNames.Num() <= 1)
		{
			return true;
		}

		// We require all curve names to be valid.
		const int32 NumCurves = CurveGroup.CurveNames.Num();
		for (int32 Index = 0; Index < NumCurves; ++Index)
		{
			const FName Name = CurveGroup.CurveNames[Index].CurveName;
			if (!Name.IsValid() || Name.IsNone())
			{
				return true;
			}

			if (!CurveNames.Contains(Name))
			{
				return true;
			}
		}
	}

	return false;
}

EMLDeformerSkinningMode UNeuralMorphInputInfo::GetSkinningMode() const
{
	return SkinningMode;
}

void UNeuralMorphInputInfo::SetSkinningMode(EMLDeformerSkinningMode Mode)
{
	SkinningMode = Mode;
}
