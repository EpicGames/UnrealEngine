// Copyright Epic Games, Inc. All Rights Reserved.
#include "NeuralMorphModel.h"
#include "NeuralMorphModelVizSettings.h"
#include "NeuralMorphModelInstance.h"
#include "NeuralMorphInputInfo.h"
#include "NeuralMorphNetwork.h"
#include "MLDeformerAsset.h"
#include "MLDeformerComponent.h"
#include "MLDeformerObjectVersion.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/UObjectGlobals.h"
#include "Engine/SkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NeuralMorphModel)

#define LOCTEXT_NAMESPACE "NeuralMorphModel"

// Implement our module.
namespace UE::NeuralMorphModel
{
	class NEURALMORPHMODEL_API FNeuralMorphModelModule
		: public IModuleInterface
	{
	};
}
IMPLEMENT_MODULE(UE::NeuralMorphModel::FNeuralMorphModelModule, NeuralMorphModel)

// Our log category for this model.
NEURALMORPHMODEL_API DEFINE_LOG_CATEGORY(LogNeuralMorphModel)

//////////////////////////////////////////////////////////////////////////////

UNeuralMorphModel::UNeuralMorphModel(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Create the visualization settings for this model.
	// Never directly create one of the frameworks base classes such as the FMLDeformerMorphModelVizSettings as
	// that can cause issues with detail customizations.
#if WITH_EDITORONLY_DATA
	SetVizSettings(ObjectInitializer.CreateEditorOnlyDefaultSubobject<UNeuralMorphModelVizSettings>(this, TEXT("VizSettings")));
#endif
}

UMLDeformerModelInstance* UNeuralMorphModel::CreateModelInstance(UMLDeformerComponent* Component)
{
	return NewObject<UNeuralMorphModelInstance>(Component);
}

UMLDeformerInputInfo* UNeuralMorphModel::CreateInputInfo()
{
	return NewObject<UNeuralMorphInputInfo>(this, NAME_None, RF_Transactional);
}

namespace
{
	int32 FindVirtualParentIndex(const FReferenceSkeleton& RefSkel, int32 BoneIndex, const TArray<FName>& IncludedBoneNames)
	{
		int32 CurBoneIndex = BoneIndex;
		while (CurBoneIndex != INDEX_NONE)
		{
			const int32 ParentIndex = RefSkel.GetParentIndex(CurBoneIndex);
			if (ParentIndex == INDEX_NONE)
			{
				break;
			}

			const FName ParentName = RefSkel.GetBoneName(ParentIndex);
			if (IncludedBoneNames.Contains(ParentName))
			{
				return ParentIndex;
			}

			CurBoneIndex = ParentIndex;
		}

		return BoneIndex;
	}

	void ModifyMaskInfosForBackwardCompatibility(const FReferenceSkeleton& RefSkel, const TArray<FBoneReference>& BoneIncludeList, TMap<FName, FMLDeformerMaskInfo>& MaskInfos)
	{
		// If we are not using some masks, just skip it.
		if (MaskInfos.IsEmpty())
		{
			return;
		}

		// Build a list of bone names that we have setup as inputs.
		TArray<FName> InputBoneNames;
		for (const FBoneReference& BoneReference : BoneIncludeList)
		{
			InputBoneNames.Add(BoneReference.BoneName);
		}

		// Build a virtual parent table.
		// This basically finds the first parent bone up the chain that is also in the bone input list.
		// So if the input list contains upper and lower arm, then the tip of the index finger's virtual parent will be the lower arm bone
		// even if there is a hand bone and more finger bones up the chain.
		TArray<int32> VirtualParentTable;
		VirtualParentTable.SetNumUninitialized(RefSkel.GetNum());
		for (int32 BoneIndex = 0; BoneIndex < RefSkel.GetNum(); ++BoneIndex)
		{
			VirtualParentTable[BoneIndex] = FindVirtualParentIndex(RefSkel, BoneIndex, InputBoneNames);
		}

		// For all bone masks.
		TArray<FName> RequiredBones;
		for (auto& MaskInfoEntry : MaskInfos)
		{
			FMLDeformerMaskInfo& MaskInfo = MaskInfoEntry.Value;
			RequiredBones.Reset();
			for (const FName MaskBoneName : MaskInfo.BoneNames)
			{
				const int32 MaskBoneIndex = RefSkel.FindBoneIndex(MaskBoneName);
				if (MaskBoneIndex == INDEX_NONE)
				{
					UE_LOGF(LogNeuralMorphModel, Warning, "Mask contains a bone named '%ls' that isn't inside the skeleton.", *MaskBoneName.ToString());
					continue;
				}
	
				// Find out all required bones.
				// This adds all child bones that have the current bone as (virtual) parent.
				if (MaskBoneIndex != INDEX_NONE)
				{
					for (int32 Index = 0; Index < VirtualParentTable.Num(); ++Index)
					{
						const int32 VirtualParent = VirtualParentTable[Index];
						const FName CurBoneName = RefSkel.GetBoneName(Index);
						if (VirtualParent == MaskBoneIndex && !MaskInfo.BoneNames.Contains(CurBoneName))
						{
							RequiredBones.AddUnique(CurBoneName);
						}
					}
				}
			}

			// Add the required bones.
			for (const FName BoneName : RequiredBones)
			{
				MaskInfo.BoneNames.AddUnique(BoneName);
			}
		}
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	FMLDeformerMaskInfo ConvertMaskInfo(const FNeuralMorphMaskInfo& NeuralMaskInfo)
	{
		FMLDeformerMaskInfo Result;
		Result.BoneNames = NeuralMaskInfo.BoneNames;
		Result.MaskMode = NeuralMaskInfo.MaskMode;
		Result.VertexAttributeName = NeuralMaskInfo.VertexAttributeName;
		return Result;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
} // Anonymous namespace.

void UNeuralMorphModel::Serialize(FArchive& Archive)
{
	TMap<FName, FMLDeformerMaskInfo> BoneMaskInfoMapBackup;
	TMap<FName, FMLDeformerMaskInfo> BoneGroupMaskInfoMapBackup;
	bool bProcessedDataOnCook = false;
	ON_SCOPE_EXIT
	{
		if (bProcessedDataOnCook && GetRecoverStrippedDataAfterCook())
		{
			BoneMaskInfoMap = MoveTemp(BoneMaskInfoMapBackup);
			BoneGroupMaskInfoMap = MoveTemp(BoneGroupMaskInfoMapBackup);
		}
	};

	if (Archive.IsSaving() && Archive.IsCooking())
	{
		// We haven't got a trained network, let's log a message about this, as it might be overlooked.
		if (NeuralMorphNetwork == nullptr)
		{
			UE_LOGF(LogNeuralMorphModel, Display, "Neural Morph Model in MLD asset '%ls' still needs to be trained.", *GetDeformerAsset()->GetName());
		}

		// Strip the mask data in the cooked asset.
		BoneMaskInfoMapBackup = MoveTemp(BoneMaskInfoMap);
		BoneGroupMaskInfoMapBackup = MoveTemp(BoneGroupMaskInfoMap);
		BoneMaskInfoMap.Empty();
		BoneGroupMaskInfoMap.Empty();
		bProcessedDataOnCook = true;
	}

	Super::Serialize(Archive);
	Archive.UsingCustomVersion(UE::MLDeformer::FMLDeformerObjectVersion::GUID);

	if (Archive.IsSaving())
	{
		UpdateMissingGroupNames();
	}

	// Convert the UMLDeformerInputInfo object into a UNeuralMorphInputInfo object for backward compatiblity.
	UMLDeformerInputInfo* CurInputInfo = GetInputInfo();
	if (CurInputInfo)
	{
		if (!CurInputInfo->IsA<UNeuralMorphInputInfo>())
		{
			UNeuralMorphInputInfo* NeuralMorphInputInfo = Cast<UNeuralMorphInputInfo>(CreateInputInfo());
			NeuralMorphInputInfo->CopyMembersFrom(CurInputInfo);
			SetInputInfo(NeuralMorphInputInfo);
		}		
 	}

	#if WITH_EDITORONLY_DATA
		// Since 5.5 we removed the FNeuralMorphMaskInfo and use the FMLDeformerMaskInfo type instead.
		// We need to convert the masks.
		if (Archive.IsLoading() && 
			Archive.CustomVer(UE::MLDeformer::FMLDeformerObjectVersion::GUID) < UE::MLDeformer::FMLDeformerObjectVersion::MaskInfoMovedToFramework)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS;
				for (const auto& MaskItemPair : BoneMaskInfos_DEPRECATED)
				{
					BoneMaskInfoMap.Add(MaskItemPair.Key, ConvertMaskInfo(MaskItemPair.Value));
				}

				for (const auto& MaskItemPair : BoneGroupMaskInfos_DEPRECATED)
				{
					BoneGroupMaskInfoMap.Add(MaskItemPair.Key, ConvertMaskInfo(MaskItemPair.Value));
				}

				BoneMaskInfos_DEPRECATED.Empty();
				BoneGroupMaskInfos_DEPRECATED.Empty();
			PRAGMA_ENABLE_DEPRECATION_WARNINGS;
		}
	#endif
}

void UNeuralMorphModel::UpdateMissingGroupNames()
{
	// Auto set names for the bone groups if they haven't been set yet.
	for (int32 Index = 0; Index < BoneGroups.Num(); ++Index)
	{
		FNeuralMorphBoneGroup& BoneGroup = BoneGroups[Index];
		if (!BoneGroup.GroupName.IsValid() || BoneGroup.GroupName.IsNone())
		{
			BoneGroup.GroupName = FName(FString::Format(TEXT("Bone Group #{0}"), {Index}));
		}
	}

	// Auto set names for the curve groups if they haven't been set yet.
	for (int32 Index = 0; Index < CurveGroups.Num(); ++Index)
	{
		FNeuralMorphCurveGroup& CurveGroup = CurveGroups[Index];
		if (!CurveGroup.GroupName.IsValid() || CurveGroup.GroupName.IsNone())
		{
			CurveGroup.GroupName = FName(FString::Format(TEXT("Curve Group #{0}"), {Index}));
		}
	}
}

void UNeuralMorphModel::PostLoad()
{
	Super::PostLoad();
	
#if WITH_EDITORONLY_DATA
	// Since 5.4 (when LOD was added), we modified the way the bone masks are generated.
	// In UE 5.3 we added some extra bones to the mask. To keep the same behavior, we will upgrade the mask to include those bones again to ensure we get the same results when training.
	// Newer assets will not do this.
	const int32 Version = GetLinkerCustomVersion(UE::MLDeformer::FMLDeformerObjectVersion::GUID);
	if (Version < UE::MLDeformer::FMLDeformerObjectVersion::LODSupportAdded)
	{
		USkeletalMesh* SkelMesh = GetSkeletalMesh();
		if (SkelMesh)
		{
			const FReferenceSkeleton& RefSkeleton = SkelMesh->GetRefSkeleton();
			ModifyMaskInfosForBackwardCompatibility(RefSkeleton, GetBoneIncludeList(), BoneMaskInfoMap);
			ModifyMaskInfosForBackwardCompatibility(RefSkeleton, GetBoneIncludeList(), BoneGroupMaskInfoMap);
		}
	}
#endif
	
	UpdateMissingGroupNames();
	check(SupportsGlobalModeOnly() ? Mode == ENeuralMorphMode::Global : true);
}

void UNeuralMorphModel::SetNeuralMorphNetwork(UNeuralMorphNetwork* Net)
{ 
	NeuralMorphNetwork = Net;
	GetReinitModelInstanceDelegate().Broadcast();
}

int32 UNeuralMorphModel::GetNumFloatsPerCurve() const
{
	return (Mode == ENeuralMorphMode::Local) ? 6 : 1;
}

bool UNeuralMorphModel::IsTrained() const
{
	return (NeuralMorphNetwork && NeuralMorphNetwork->GetMainModel() != nullptr);
}

void UNeuralMorphModel::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	const UNeuralMorphInputInfo* NeuralInputInfo = Cast<UNeuralMorphInputInfo>(GetInputInfo());
	if (NeuralInputInfo)
	{
		Context.AddTag(FAssetRegistryTag("MLDeformer.NeuralMorphModel.Trained.NumBoneGroups", FString::FromInt(NeuralInputInfo->GetBoneGroups().Num()), FAssetRegistryTag::TT_Numerical));
		Context.AddTag(FAssetRegistryTag("MLDeformer.NeuralMorphModel.Trained.NumCurveGroups", FString::FromInt(NeuralInputInfo->GetCurveGroups().Num()), FAssetRegistryTag::TT_Numerical));
	}

	Context.AddTag(FAssetRegistryTag("MLDeformer.NeuralMorphModel.Mode", Mode == ENeuralMorphMode::Local ? TEXT("Local") : TEXT("Global"), FAssetRegistryTag::TT_Alphabetical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.NeuralMorphModel.NumMorphsPerBone", FString::FromInt(LocalNumMorphTargetsPerBone), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.NeuralMorphModel.NumMorphTargets", FString::FromInt(GlobalNumMorphTargets), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.NeuralMorphModel.NumIterations", FString::FromInt(NumIterations), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.NeuralMorphModel.LocalNumHiddenLayers", FString::FromInt(LocalNumHiddenLayers), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.NeuralMorphModel.LocalNumNeuronsPerLayer", FString::FromInt(LocalNumNeuronsPerLayer), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.NeuralMorphModel.GlobalNumHiddenLayers", FString::FromInt(GlobalNumHiddenLayers), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.NeuralMorphModel.GlobalNumNeuronsPerLayer", FString::FromInt(GlobalNumNeuronsPerLayer), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.NeuralMorphModel.BatchSize", FString::FromInt(BatchSize), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.NeuralMorphModel.LearningRate", FString::Printf(TEXT("%f"), LearningRate), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.NeuralMorphModel.RegularizationFactor", FString::Printf(TEXT("%f"), RegularizationFactor), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.NeuralMorphModel.SmoothLossBeta", FString::Printf(TEXT("%f"), SmoothLossBeta), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.NeuralMorphModel.EnableBoneMasks", bEnableBoneMasks ? TEXT("True") : TEXT("False"), FAssetRegistryTag::TT_Alphabetical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.NeuralMorphModel.UseDualQuaternions", SkinningMode == EMLDeformerSkinningMode::DualQuaternion ? TEXT("True") : TEXT("False"), FAssetRegistryTag::TT_Alphabetical));
}

#undef LOCTEXT_NAMESPACE
