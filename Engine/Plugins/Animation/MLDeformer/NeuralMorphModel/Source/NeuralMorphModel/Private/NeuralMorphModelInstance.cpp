// Copyright Epic Games, Inc. All Rights Reserved.

#include "NeuralMorphModelInstance.h"
#include "NeuralMorphModel.h"
#include "NeuralMorphNetwork.h"
#include "NeuralMorphInputInfo.h"
#include "MLDeformerComponent.h"
#include "MLDeformerAsset.h"
#include "Components/ExternalMorphSet.h"
#include "Components/SkeletalMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NeuralMorphModelInstance)

CSV_DECLARE_CATEGORY_MODULE_EXTERN(MLDEFORMERFRAMEWORK_API, MLDeformer);

void UNeuralMorphModelInstance::Init(USkeletalMeshComponent* SkelMeshComponent)
{
	Super::Init(SkelMeshComponent);

	UNeuralMorphModel* MorphModel = Cast<UNeuralMorphModel>(Model);
	UNeuralMorphNetwork* MorphNetwork = MorphModel->GetNeuralMorphNetwork();
	NetworkInstance = MorphNetwork ? MorphNetwork->CreateInstance() : nullptr;

	UNeuralMorphInputInfo* InputInfo = Cast<UNeuralMorphInputInfo>(Model->GetInputInfo());
	if (InputInfo == nullptr)
	{
		return;
	}

	InputInfo->GenerateBoneGroupIndices(BoneGroupIndices);
	InputInfo->GenerateCurveGroupIndices(CurveGroupIndices);
}

void UNeuralMorphModelInstance::FillNetworkInputs()
{
	check(NetworkInstance);
	UNeuralMorphNetwork* MorphNetwork = Cast<UNeuralMorphNetwork>(NetworkInstance->GetOuter());

	// Write the input bone transforms.
	float* MainInputs = NetworkInstance->GetInputs().GetData();
	const int32 NumMainNeuralNetworkInputs = MorphNetwork->GetNumMainInputs();
	Super::SetBoneTransforms(MainInputs, NumMainNeuralNetworkInputs, 0);

	// Write the curve input values, start writing after the bone inputs.
	const int32 CurveWriteOffset = Model->GetInputInfo()->GetNumBones() * Model->GetNumFloatsPerBone();
	SetCurveValues(MainInputs, NumMainNeuralNetworkInputs, CurveWriteOffset);

	// Normalize the inputs.
	const float* Means = MorphNetwork->GetInputMeans().GetData();
	const float* Stds = MorphNetwork->GetInputStds().GetData();
	const int32 NumInputs = MorphNetwork->GetInputMeans().Num();
	for (int32 Index = 0; Index < NumInputs; ++Index)
	{
		MainInputs[Index] = (MainInputs[Index] - Means[Index]) / Stds[Index];
	}

	// Write the input transforms for the group Neural Network.
	// We reuse the same input values from the main network, and those values just got normalized.
	// So there is no need to normalize them again.
	if (MorphNetwork->GetGroupModel())
	{
		const int32 NumFloatsPerBone = Model->GetNumFloatsPerBone();

		// Write the bone transforms.
		int32 Offset = 0;
		float* GroupNetworkInputs = NetworkInstance->GetGroupInputs().GetData();
		const int32 NumBoneGroups = BoneGroupIndices.Num();
		for (int32 Index = 0; Index < BoneGroupIndices.Num(); ++Index)
		{
			const int32 BoneIndex = BoneGroupIndices[Index];
			if (BoneIndex != INDEX_NONE)
			{
				const int32 BoneOffset = BoneIndex * NumFloatsPerBone;
				GroupNetworkInputs[Offset++] = MainInputs[BoneOffset];
				GroupNetworkInputs[Offset++] = MainInputs[BoneOffset + 1];
				GroupNetworkInputs[Offset++] = MainInputs[BoneOffset + 2];
				GroupNetworkInputs[Offset++] = MainInputs[BoneOffset + 3];
				GroupNetworkInputs[Offset++] = MainInputs[BoneOffset + 4];
				GroupNetworkInputs[Offset++] = MainInputs[BoneOffset + 5];
			}
			else
			{
				GroupNetworkInputs[Offset++] = 0.0f;
				GroupNetworkInputs[Offset++] = 0.0f;
				GroupNetworkInputs[Offset++] = 0.0f;
				GroupNetworkInputs[Offset++] = 0.0f;
				GroupNetworkInputs[Offset++] = 0.0f;
				GroupNetworkInputs[Offset++] = 0.0f;
			}
		}

		// Write the curve values for all curve groups.
		const int32 NumFloatsPerCurve = MorphNetwork->GetNumFloatsPerCurve();
		const int32 NumCurveGroups = CurveGroupIndices.Num();
		for (int32 Index = 0; Index < NumCurveGroups; ++Index)
		{
			const int32 CurveIndex = CurveGroupIndices[Index];
			if (CurveIndex != INDEX_NONE)
			{
				const int32 CurveOffset = CurveWriteOffset + (CurveIndex * NumFloatsPerCurve);
				GroupNetworkInputs[Offset++] = MainInputs[CurveOffset];
				GroupNetworkInputs[Offset++] = MainInputs[CurveOffset + 1];
				GroupNetworkInputs[Offset++] = MainInputs[CurveOffset + 2];
				GroupNetworkInputs[Offset++] = MainInputs[CurveOffset + 3];
				GroupNetworkInputs[Offset++] = MainInputs[CurveOffset + 4];
				GroupNetworkInputs[Offset++] = MainInputs[CurveOffset + 5];
			}
			else
			{
				GroupNetworkInputs[Offset++] = 0.0f;
				GroupNetworkInputs[Offset++] = 0.0f;
				GroupNetworkInputs[Offset++] = 0.0f;
				GroupNetworkInputs[Offset++] = 0.0f;
				GroupNetworkInputs[Offset++] = 0.0f;
				GroupNetworkInputs[Offset++] = 0.0f;
			}
		}
	}
}

// This will fill the curve values for the curves, with curve groups excluded.
int64 UNeuralMorphModelInstance::SetCurveValues(float* OutputBuffer, int64 OutputBufferSize, int64 StartIndex)
{
	UNeuralMorphModel* MorphModel = Cast<UNeuralMorphModel>(Model);
	UNeuralMorphNetwork* MorphNetwork = MorphModel->GetNeuralMorphNetwork();

	const UMLDeformerInputInfo* InputInfo = Model->GetInputInfo();

	int64 Index = StartIndex;
	const int32 AssetNumCurves = InputInfo->GetNumCurves();
	const int32 NumFloatsPerCurve = MorphNetwork->GetNumFloatsPerCurve();
	const int32 NumCurveFloats = AssetNumCurves * NumFloatsPerCurve;
	checkf((Index + NumCurveFloats) <= OutputBufferSize, TEXT("Writing curves past the end of the input buffer"));

	if (NumFloatsPerCurve > 1)
	{
		// First write all zeros.
		for (int32 CurveIndex = 0; CurveIndex < NumCurveFloats; ++CurveIndex)
		{
			OutputBuffer[Index++] = 0.0f;
		}

		// Write the actual curve values.
		for (int32 CurveIndex = 0; CurveIndex < AssetNumCurves; ++CurveIndex)
		{
			const FName CurveName = InputInfo->GetCurveName(CurveIndex);
			float CurveValue;
			SkeletalMeshComponent->GetCurveValue(CurveName, 0.0f, CurveValue);
			OutputBuffer[StartIndex + CurveIndex * NumFloatsPerCurve] = CurveValue;
		}
	}
	else
	{
		checkSlow(NumFloatsPerCurve == 1);
		for (int32 CurveIndex = 0; CurveIndex < AssetNumCurves; ++CurveIndex)
		{
			const FName CurveName = InputInfo->GetCurveName(CurveIndex);
			float CurveValue;
			SkeletalMeshComponent->GetCurveValue(CurveName, 0.0f, CurveValue);
			OutputBuffer[Index++] = CurveValue;
		}
	}

	return Index;
}


bool UNeuralMorphModelInstance::SetupInputs()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UNeuralMorphModelInstance::SetupInputs)

	// Make sure we have a valid network.
	UNeuralMorphModel* MorphModel = Cast<UNeuralMorphModel>(Model);
	UNeuralMorphNetwork* MorphNetwork = MorphModel->GetNeuralMorphNetwork();
	if (MorphNetwork == nullptr || MorphNetwork->GetMainModel() == nullptr)
	{		
		return false;
	}

	// Some additional safety checks.
	if (SkeletalMeshComponent == nullptr ||
		SkeletalMeshComponent->GetSkeletalMeshAsset() == nullptr ||
		!bIsCompatible)
	{
		return false;
	}

	// If the neural network expects a different number of inputs, do nothing.
	const int64 NumNeuralNetInputs = MorphNetwork->GetNumInputs();
	const int32 NumFloatsPerBone = 6;
	const int32 NumFloatsPerCurve = MorphNetwork->GetNumFloatsPerCurve();
	const int64 NumDeformerAssetInputs = Model->GetInputInfo()->CalcNumNeuralNetInputs(NumFloatsPerBone, NumFloatsPerCurve);
	if (NumNeuralNetInputs != NumDeformerAssetInputs)
	{
		return false;
	}

	// Set the actual network input values.
	FillNetworkInputs();

	return true;
}

void UNeuralMorphModelInstance::Execute(float ModelWeight)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UNeuralMorphModelInstance::Execute)
	CSV_SCOPED_TIMING_STAT(MLDeformer, NeuralMorphExecute);

	UNeuralMorphModel* MorphModel = Cast<UNeuralMorphModel>(Model);
	UNeuralMorphNetwork* MorphNetwork = MorphModel->GetNeuralMorphNetwork();
	if (MorphNetwork == nullptr)
	{
		Super::Execute(ModelWeight);
		return;
	}

	// Grab the weight data for this morph set.
	// This could potentially fail if we are applying this deformer to the wrong skeletal mesh component.
	const int32 LOD = SkeletalMeshComponent->GetPredictedLODLevel();
	FExternalMorphSetWeights* WeightData = FindWeightData(LOD);
	if (WeightData == nullptr)
	{
		return;
	}

	// Make sure we have a valid network instance.
	if (!NetworkInstance || MorphNetwork->IsEmpty())
	{
		WeightData->ZeroWeights();
		return;
	}

	// If our model is active, we want to run the neural network and update the morph weights
	// with the values that the neural net calculated for us.
	// Get the network output values, read the values and use them as morph target weights inside the skeletal mesh component.
	const TArrayView<const float> NetworkOutputs = NetworkInstance->GetOutputs();
	const int32 NumNetworkWeights = NetworkOutputs.Num();
	const int32 NumMorphWeights = WeightData->Weights.Num();

	// Set the weights to zero if we don't have enough weights.
	if (NumMorphWeights < NumNetworkWeights + 1)	// Plus 1 because of the extra means morph target.
	{
		WeightData->ZeroWeights();
		return;
	}

	// Perform inference on the neural network, this updates its output values.
	NetworkInstance->Run();

	// Set the first morph target, which represents the means, to a weight of 1.0, as it always needs to be fully active.
	WeightData->Weights[0] = ModelWeight;

	// Update all generated morph target weights with the values calculated by our neural network.
	for (int32 MorphIndex = 0; MorphIndex < NumNetworkWeights; ++MorphIndex)
	{
		WeightData->Weights[MorphIndex + 1] = NetworkOutputs[MorphIndex] * ModelWeight;
	}

	// Clamp morph target weights to be within the bounds we seen on the training data set.
	// This can prevent 'exploding' weight values when we have network inputs that are far away from inputs we seen during training.
	if (MorphModel->IsMorphWeightClampingEnabled())
	{
		// Start at weights[1], because the first item contains the morph weight that represents the mean.
		// And we don't want to clamp that one as it cannot go out of bounds, as it is basically always 1.0 if the deformer is fully active.
		// The ClampMorphTargetWeights method will modify the values in the array.
		TArrayView<float> WeightsArray(&WeightData->Weights[1], NumNetworkWeights);
		MorphModel->ClampMorphTargetWeights(WeightsArray);
	}
}
