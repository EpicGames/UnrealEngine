// Copyright Epic Games, Inc. All Rights Reserved.

#include "NeuralMorphNetwork.h"
#include "NeuralMorphModel.h"
#include "UObject/SoftObjectPath.h"
#include "HAL/FileManagerGeneric.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/FileHelper.h"

#include "NNE.h"
#include "NNERuntime.h"
#include "NNERuntimeCPU.h"
#include "NNEModelData.h"
#include "NNERuntimeBasicCpuBuilder.h"

//--------------------------------------------------------------------------
// UNeuralMorphMLP
//--------------------------------------------------------------------------

#include UE_INLINE_GENERATED_CPP_BY_NAME(NeuralMorphNetwork)

PRAGMA_DISABLE_DEPRECATION_WARNINGS

void UNeuralMorphMLP::Empty()
{
	Layers.Empty();
}

bool UNeuralMorphMLP::IsEmpty() const
{
	return GetNumLayers() == 0;
}

int32 UNeuralMorphMLP::GetNumInputs() const
{
	return !Layers.IsEmpty() ? Layers[0]->NumInputs * Layers[0]->Depth : 0;
}

int32 UNeuralMorphMLP::GetNumOutputs() const
{
	return !Layers.IsEmpty() ? Layers[Layers.Num() - 1]->NumOutputs * Layers[Layers.Num() - 1]->Depth : 0;
}

int32 UNeuralMorphMLP::GetMaxNumLayerInputs() const
{
	int32 MaxValue = 0;
	for (UNeuralMorphMLPLayer* Layer : Layers)
	{
		MaxValue = FMath::Max<int32>(MaxValue, Layer->NumInputs * Layer->Depth);
	}
	return MaxValue;
}

int32 UNeuralMorphMLP::GetMaxNumLayerOutputs() const
{
	int32 MaxValue = 0;
	for (UNeuralMorphMLPLayer* Layer : Layers)
	{
		MaxValue = FMath::Max<int32>(MaxValue, Layer->NumOutputs * Layer->Depth);
	}
	return MaxValue;
}

UNeuralMorphMLPLayer* UNeuralMorphMLP::LoadNetworkLayer(FArchive& FileReader)
{
	UNeuralMorphMLPLayer* Layer = NewObject<UNeuralMorphMLPLayer>(this);

	struct FLayerHeader
	{
		int32 NumInputs;
		int32 NumOutputs;
		int32 NumWeights;
		int32 NumBiases;
	};

	FLayerHeader LayerHeader;
	FileReader.Serialize(&LayerHeader, sizeof(FLayerHeader));
	if (FileReader.IsError())
	{
		UE_LOGF(LogNeuralMorphModel, Error, "Failed to load a layer header for the network!");
		Layer->ConditionalBeginDestroy();
		return nullptr;
	}

	Layer->NumInputs = LayerHeader.NumInputs;
	Layer->NumOutputs = LayerHeader.NumOutputs;
	check(LayerHeader.NumInputs * LayerHeader.NumOutputs > 0);
	Layer->Depth = LayerHeader.NumWeights / (LayerHeader.NumInputs * LayerHeader.NumOutputs);

	Layer->Weights.AddZeroed(LayerHeader.NumWeights);
	FileReader.Serialize(Layer->Weights.GetData(), LayerHeader.NumWeights * sizeof(float));
	if (FileReader.IsError())
	{
		UE_LOGF(LogNeuralMorphModel, Error, "Failed to load layer weights of the network!");
		Layer->ConditionalBeginDestroy();
		return nullptr;
	}

	Layer->Biases.AddZeroed(LayerHeader.NumBiases);
	FileReader.Serialize(Layer->Biases.GetData(), LayerHeader.NumBiases * sizeof(float));
	if (FileReader.IsError())
	{
		UE_LOGF(LogNeuralMorphModel, Error, "Failed to load layer biases of the network!");
		Layer->ConditionalBeginDestroy();
		return nullptr;
	}

	return Layer;
}


bool UNeuralMorphMLP::Load(FArchive& FileReader)
{
	Empty();

	// Read the FOURCC, to identify the file type.
	char FOURCC[4] {' ', ' ', ' ', ' '};
	FileReader.Serialize(FOURCC, 4);
	if (FileReader.IsError())
	{
		UE_LOGF(LogNeuralMorphModel, Error, "Failed to read the FOURCC!");
		Empty();
		return false;
	}

	if (FOURCC[0] != 'N' || FOURCC[1] != 'M' || FOURCC[2] != 'L' || FOURCC[3] != 'P')	// NMLP (Neural MLP)
	{
		UE_LOGF(LogNeuralMorphModel, Error, "The file is not a valid valid neural morph MLP file type!");
		Empty();
		return false;
	}

	// Load the version number.
	int32 Version = -1;
	FileReader.Serialize(&Version, sizeof(int32));
	if (FileReader.IsError())
	{
		UE_LOGF(LogNeuralMorphModel, Error, "Failed to load version");
		Empty();
		return false;
	}

	if (Version != 1)
	{
		UE_LOGF(LogNeuralMorphModel, Error, "The Neural Morph MLP file '%ls' is of an unknown version (Version=%d)!", *FileReader.GetArchiveName(), Version);
		Empty();
		return false;
	}

	int32 NumLayers = -1;
	FileReader.Serialize(&NumLayers, sizeof(int32));
	if (FileReader.IsError())
	{
		UE_LOGF(LogNeuralMorphModel, Error, "Failed to load number of layers");
		Empty();
		return false;
	}

	// Load the weights and biases of the main network.
	for (int32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
	{
		UNeuralMorphMLPLayer* Layer = LoadNetworkLayer(FileReader);
		if (Layer == nullptr)
		{
			Empty();
			return false;
		}
		Layers.Add(Layer);
		UE_LOGF(LogNeuralMorphModel, Verbose, "MLP Layer %d --> NumWeights=%d (%dx%dx%d)   NumBiases=%d", LayerIndex, Layer->Weights.Num(), Layer->NumInputs, Layer->NumOutputs, Layer->Depth, Layer->Biases.Num());
	}

	UE_LOGF(LogNeuralMorphModel, Display, "Successfully loaded neural morph MLP from file '%ls'", *FileReader.GetArchiveName());
	return true;
}

int32 UNeuralMorphMLP::GetNumLayers() const
{
	return Layers.Num();
}

UNeuralMorphMLPLayer& UNeuralMorphMLP::GetLayer(int32 Index) const
{
	return *Layers[Index].Get();
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

//--------------------------------------------------------------------------
// UNeuralMorphNetwork
//--------------------------------------------------------------------------

namespace UE::NeuralMorphModel::Private
{
	// Magic and version number used to identify this file data transfered between
	// python and the NeuralMorphModel. This should match what is in "nmm_shared.py"
	static constexpr uint32 MagicNumber = 0x234A1304;
	static constexpr uint32 VersionNumber = 1;

	namespace Serialization
	{
		//--------------------------------------------------------------------------

		static inline void Align(uint64& InOutOffset, uint32 Alignment)
		{
			InOutOffset = ((InOutOffset + Alignment - 1) / Alignment) * Alignment;
		}

		//--------------------------------------------------------------------------

		static inline void Size(uint64& InOutOffset, const uint32& In)
		{
			Align(InOutOffset, sizeof(uint32));
			InOutOffset += sizeof(uint32);
		}

		static inline void Size(uint64& InOutOffset, const float& In)
		{
			Align(InOutOffset, alignof(float));
			InOutOffset += sizeof(float);
		}

		static inline void Size(uint64& InOutOffset, const TConstArrayView<float> In)
		{
			Align(InOutOffset, 64);
			InOutOffset += In.Num() * sizeof(float);
		}

		static inline void Size(uint64& InOutOffset, const TConstArrayView<uint16> In)
		{
			Align(InOutOffset, 64);
			InOutOffset += In.Num() * sizeof(uint16);
		}

		//--------------------------------------------------------------------------

		static inline void Load(uint64& InOutOffset, uint32& Out, TConstArrayView<uint8> Data)
		{
			Align(InOutOffset, sizeof(uint32));
			Out = *((uint32*)&Data[InOutOffset]);
			InOutOffset += sizeof(uint32);
		}

		static inline void Load(uint64& InOutOffset, int32& Out, TConstArrayView<uint8> Data)
		{
			Align(InOutOffset, sizeof(int32));
			Out = *((int32*)&Data[InOutOffset]);
			InOutOffset += sizeof(int32);
		}

		static inline void Load(uint64& InOutOffset, float& Out, TConstArrayView<uint8> Data)
		{
			Align(InOutOffset, sizeof(float));
			Out = *((float*)&Data[InOutOffset]);
			InOutOffset += sizeof(float);
		}

		static inline void Load(uint64& InOutOffset, TArrayView<float> Out, TConstArrayView<uint8> Data)
		{
			Align(InOutOffset, 64);
			FMemory::Memcpy(Out.GetData(), &Data[InOutOffset], Out.Num() * sizeof(float));
			Out = MakeArrayView<float>((float*)&Data[InOutOffset], Out.Num());
			InOutOffset += Out.Num() * sizeof(float);
		}

		static inline void Load(uint64& InOutOffset, FString& Out, TConstArrayView<uint8> Data)
		{
			uint32 Num;
			Load(InOutOffset, Num, Data);
			Out = UTF8_TO_TCHAR(&Data[InOutOffset]);
			InOutOffset += Num;
		}
	}

	//--------------------------------------------------------------------------

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	static inline void ConvertMLPToFileData(TArray<uint8>& OutBytes, TObjectPtr<UNeuralMorphMLP>& MLP)
	{
		UE::NNE::RuntimeBasic::FModelBuilder Builder;

		const uint32 LayerNum = MLP->GetNumLayers();

		TArray<UE::NNE::RuntimeBasic::FModelBuilderElement, TInlineAllocator<32>> LayerElements;
		LayerElements.Reserve(2 * LayerNum);

		for (uint32 LayerIdx = 0; LayerIdx < LayerNum; LayerIdx++)
		{
			const uint32 InputSize = MLP->GetLayer(LayerIdx).NumInputs;
			const uint32 OutputSize = MLP->GetLayer(LayerIdx).NumOutputs;
			const uint32 BlockNum = MLP->GetLayer(LayerIdx).Depth;
			const TConstArrayView<float> Weights = MLP->GetLayer(LayerIdx).Weights;
			const TConstArrayView<float> Biases = MLP->GetLayer(LayerIdx).Biases;

			if (BlockNum == 1)
			{
				LayerElements.Add(Builder.MakeLinear(InputSize, OutputSize, Weights, Biases));
			}
			else
			{
				LayerElements.Add(Builder.MakeMultiLinear(InputSize, OutputSize, BlockNum, Weights, Biases));
			}

			LayerElements.Add(Builder.MakeELU(OutputSize * BlockNum));
		}

		uint32 InputSize, OutputSize;
		Builder.WriteFileDataAndReset(OutBytes, InputSize, OutputSize, Builder.MakeSequence(LayerElements));

		MLP = nullptr;
	}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

UNeuralMorphNetwork::UNeuralMorphNetwork() : Super() {}
UNeuralMorphNetwork::UNeuralMorphNetwork(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer) {}
UNeuralMorphNetwork::UNeuralMorphNetwork(FVTableHelper& Helper) : Super(Helper) {}
UNeuralMorphNetwork::~UNeuralMorphNetwork() {}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

void UNeuralMorphNetwork::PostLoad()
{
	Super::PostLoad();

	if (MainMLP_DEPRECATED)
	{
		MainMLP_DEPRECATED->ConditionalPostLoad();

		// If we are in Global mode and loading the legacy network format we need to Load NumMorphs 
		// from old network data output size since it was not stored directly in the object before.
		if (Mode == ENeuralMorphMode::Global)
		{
			NumMorphs = MainMLP_DEPRECATED->GetNumOutputs();
		}

		TArray<uint8> FileData;
		UE::NeuralMorphModel::Private::ConvertMLPToFileData(FileData, MainMLP_DEPRECATED);

		if (!MainModelData)
		{
			MainModelData = NewObject<UNNEModelData>(this);
		}

		MainModelData->Init(TEXT("ubnne"), FileData);

		FileData.Empty();
	}

	if (GroupMLP_DEPRECATED)
	{
		GroupMLP_DEPRECATED->ConditionalPostLoad();

		TArray<uint8> FileData;
		UE::NeuralMorphModel::Private::ConvertMLPToFileData(FileData, GroupMLP_DEPRECATED);

		if (!GroupModelData)
		{
			GroupModelData = NewObject<UNNEModelData>(this);
		}

		GroupModelData->Init(TEXT("ubnne"), FileData);

		FileData.Empty();
	}

	// Create models

	ensureMsgf(FModuleManager::Get().LoadModule(TEXT("NNERuntimeBasicCpu")), TEXT("Unable to load module for NNE runtime NNERuntimeBasicCpu."));
	
	TWeakInterfacePtr<INNERuntimeCPU> RuntimeCPU = UE::NNE::GetRuntime<INNERuntimeCPU>(TEXT("NNERuntimeBasicCpu"));

	if (ensureMsgf(RuntimeCPU.IsValid(), TEXT("Could not find requested NNE Runtime")))
	{
		if (MainModelData)
		{
			MainModel = RuntimeCPU->CreateModelCPU(MainModelData);
		}

		if (GroupModelData)
		{
			GroupModel = RuntimeCPU->CreateModelCPU(GroupModelData);
		}
	}

	// If we are not in the editor then we clear the FileData and FileType since these will be
	// using additional memory if we are loading from the legacy format.

#if !WITH_EDITOR
	if (MainModelData)
	{
		MainModelData->ClearFileDataAndFileType();
	}

	if (GroupModelData)
	{
		GroupModelData->ClearFileDataAndFileType();
	}
#endif
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UNeuralMorphNetwork::Empty()
{
	InputMeans.Empty();
	InputStd.Empty();

	if (MainModelData)
	{
		MainModelData->ConditionalBeginDestroy();
		MainModelData = nullptr;
		MainModel.Reset();
	}

	if (GroupModelData)
	{
		GroupModelData->ConditionalBeginDestroy();
		GroupModelData = nullptr;
		GroupModel.Reset();
	}

	Mode = ENeuralMorphMode::Global;
	NumMorphs = 0;
	NumMorphsPerBone = 0;
	NumBones = 0;
	NumCurves = 0;
	NumGroups = 0;
	NumItemsPerGroup = 0;
	NumFloatsPerCurve = 1;
}

bool UNeuralMorphNetwork::IsEmpty() const
{
	return !MainModel;
}

bool UNeuralMorphNetwork::Load(const FString& Filename)
{
	UE_LOGF(LogNeuralMorphModel, Display, "Loading Neural Morph Network from file '%ls'", *Filename);
	
	Empty();

	TArray<uint8> FileData;

	if (FFileHelper::LoadFileToArray(FileData, *Filename))
	{
		uint64 Offset = 0;

		if (FileData.Num() < sizeof(uint32) * 2)
		{
			UE_LOGF(LogNeuralMorphModel, Error, "The file is not a valid neural morph network file!");
			return false;
		}

		uint32 Magic;
		UE::NeuralMorphModel::Private::Serialization::Load(Offset, Magic, FileData);
		if (Magic != UE::NeuralMorphModel::Private::MagicNumber)
		{
			UE_LOGF(LogNeuralMorphModel, Error, "The file is not a valid neural morph network file!");
			return false;
		}

		uint32 Version;
		UE::NeuralMorphModel::Private::Serialization::Load(Offset, Version, FileData);
		if (Version != UE::NeuralMorphModel::Private::VersionNumber)
		{
			UE_LOGF(LogNeuralMorphModel, Error, "The file is not a supported version of neural morph network file!");
			return false;
		}

		// Load General Info

		uint32 ModeValue;
		UE::NeuralMorphModel::Private::Serialization::Load(Offset, ModeValue, FileData);
		Mode = (ModeValue == 0) ? ENeuralMorphMode::Local : ENeuralMorphMode::Global;
		UE::NeuralMorphModel::Private::Serialization::Load(Offset, NumMorphs, FileData);
		UE::NeuralMorphModel::Private::Serialization::Load(Offset, NumMorphsPerBone, FileData);
		UE::NeuralMorphModel::Private::Serialization::Load(Offset, NumBones, FileData);
		UE::NeuralMorphModel::Private::Serialization::Load(Offset, NumCurves, FileData);
		UE::NeuralMorphModel::Private::Serialization::Load(Offset, NumGroups, FileData);
		UE::NeuralMorphModel::Private::Serialization::Load(Offset, NumItemsPerGroup, FileData);
		UE::NeuralMorphModel::Private::Serialization::Load(Offset, NumFloatsPerCurve, FileData);

		// Load Mean / Std

		InputMeans.SetNumUninitialized(GetNumInputs());
		InputStd.SetNumUninitialized(GetNumInputs());

		UE::NeuralMorphModel::Private::Serialization::Load(Offset, InputMeans, FileData);
		UE::NeuralMorphModel::Private::Serialization::Load(Offset, InputStd, FileData);

		// Load NNE Runtime Name

		FString RuntimeName;
		UE::NeuralMorphModel::Private::Serialization::Load(Offset, RuntimeName, FileData);

		ensureMsgf(RuntimeName == TEXT("NNERuntimeBasicCpu"), TEXT("Currently only NNERuntimeBasicCpu runtime is supported"));

		// Load Main Network

		if (!MainModelData)
		{
			MainModelData = NewObject<UNNEModelData>(this);
		}
		
		uint32 MainModelSize;
		UE::NeuralMorphModel::Private::Serialization::Load(Offset, MainModelSize, FileData);
		UE::NeuralMorphModel::Private::Serialization::Align(Offset, 64);
		MainModelData->Init(TEXT("ubnne"), TConstArrayView<uint8>(FileData).Slice(Offset, MainModelSize));

		Offset += MainModelSize;

		// If we're in local mode and we have groups, load Group Network

		if (NumGroups > 0 && Mode == ENeuralMorphMode::Local)
		{
			if (!GroupModelData)
			{
				GroupModelData = NewObject<UNNEModelData>(this);
			}
			
			uint32 GroupModelSize;
			UE::NeuralMorphModel::Private::Serialization::Load(Offset, GroupModelSize, FileData);
			UE::NeuralMorphModel::Private::Serialization::Align(Offset, 64);
			GroupModelData->Init(TEXT("ubnne"), TConstArrayView<uint8>(FileData).Slice(Offset, GroupModelSize));

			Offset += GroupModelSize;
		}

		// Create models

		ensureMsgf(FModuleManager::Get().LoadModule(TEXT("NNERuntimeBasicCpu")), TEXT("Unable to load module for NNE runtime NNERuntimeBasicCpu."));

		TWeakInterfacePtr<INNERuntimeCPU> RuntimeCPU = UE::NNE::GetRuntime<INNERuntimeCPU>(TEXT("NNERuntimeBasicCpu"));

		if (ensureMsgf(RuntimeCPU.IsValid(), TEXT("Could not find requested NNE Runtime")))
		{
			if (MainModelData)
			{
				MainModel = RuntimeCPU->CreateModelCPU(MainModelData);
			}

			if (GroupModelData)
			{
				GroupModel = RuntimeCPU->CreateModelCPU(GroupModelData);
			}
		}

		// Done!

		checkf(Offset == FileData.Num(), TEXT("Did not fully parse file contents"));

		UE_LOGF(LogNeuralMorphModel, Display, "Successfully loaded neural morph network from file '%ls'", *Filename);
		return true;
	}
	else
	{
		UE_LOGF(LogNeuralMorphModel, Error, "Failed to read file '%ls'!", *Filename);
		return false;
	}
}

UNeuralMorphNetworkInstance* UNeuralMorphNetwork::CreateInstance()
{
	UNeuralMorphNetworkInstance* Instance = NewObject<UNeuralMorphNetworkInstance>(this);
	Instance->Init(this);
	return Instance;
}

int32 UNeuralMorphNetwork::GetNumBones() const
{
	return NumBones;
}

int32 UNeuralMorphNetwork::GetNumCurves() const
{
	return NumCurves;
}

int32 UNeuralMorphNetwork::GetNumMorphsPerBone() const
{
	return NumMorphsPerBone;
}

ENeuralMorphMode UNeuralMorphNetwork::GetMode() const
{
	return Mode;
}

const TArrayView<const float> UNeuralMorphNetwork::GetInputMeans() const
{
	return InputMeans;
}

const TArrayView<const float> UNeuralMorphNetwork::GetInputStds() const
{
	return InputStd;
}

int32 UNeuralMorphNetwork::GetNumFloatsPerCurve() const
{
	return NumFloatsPerCurve;
}

int32 UNeuralMorphNetwork::GetNumGroups() const
{
	return NumGroups;
}

int32 UNeuralMorphNetwork::GetNumItemsPerGroup() const
{
	return NumItemsPerGroup;
}

int32 UNeuralMorphNetwork::GetNumMainInputs() const
{
	return Mode == ENeuralMorphMode::Local ?
		(NumBones * 6) + (NumCurves * 6) :
		(NumBones * 6) + NumCurves;
}

int32 UNeuralMorphNetwork::GetNumMainOutputs() const
{
	return Mode == ENeuralMorphMode::Local ?
		NumMorphsPerBone * (NumBones + NumCurves) :
		NumMorphs;
}

int32 UNeuralMorphNetwork::GetNumGroupInputs() const
{
	return Mode == ENeuralMorphMode::Local ?
		NumGroups * 6 * NumItemsPerGroup :
		0;
}

int32 UNeuralMorphNetwork::GetNumGroupOutputs() const
{
	return Mode == ENeuralMorphMode::Local ?
		NumMorphsPerBone * NumGroups :
		0;
}

int32 UNeuralMorphNetwork::GetNumInputs() const
{
	return GetNumMainInputs();
}

int32 UNeuralMorphNetwork::GetNumOutputs() const
{
	return GetNumMainOutputs() + GetNumGroupOutputs();
}

UE::NNE::IModelCPU* UNeuralMorphNetwork::GetMainModel() const
{
	return MainModel.Get();
}

UE::NNE::IModelCPU* UNeuralMorphNetwork::GetGroupModel() const
{
	return GroupModel.Get();
}

//--------------------------------------------------------------------------
// UNeuralMorphNetworkInstance
//--------------------------------------------------------------------------

UNeuralMorphNetworkInstance::UNeuralMorphNetworkInstance() : Super() {}
UNeuralMorphNetworkInstance::UNeuralMorphNetworkInstance(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer) {}
UNeuralMorphNetworkInstance::UNeuralMorphNetworkInstance(FVTableHelper& Helper) : Super(Helper) {}
UNeuralMorphNetworkInstance::~UNeuralMorphNetworkInstance() {}

TArrayView<float> UNeuralMorphNetworkInstance::GetInputs()
{ 
	return TArrayView<float>(Inputs.GetData(), Inputs.Num());
}

TArrayView<const float> UNeuralMorphNetworkInstance::GetInputs() const
{ 
	return TArrayView<const float>(Inputs.GetData(), Inputs.Num());
}

TArrayView<float> UNeuralMorphNetworkInstance::GetGroupInputs()
{ 
	return TArrayView<float>(GroupInputs.GetData(), GroupInputs.Num());
}

TArrayView<const float> UNeuralMorphNetworkInstance::GetGroupInputs() const
{ 
	return TArrayView<const float>(GroupInputs.GetData(), GroupInputs.Num());
}

TArrayView<float> UNeuralMorphNetworkInstance::GetOutputs()
{ 
	return TArrayView<float>(Outputs.GetData(), Outputs.Num());
}

TArrayView<const float> UNeuralMorphNetworkInstance::GetOutputs() const
{ 
	return TArrayView<const float>(Outputs.GetData(), Outputs.Num());
}

const UNeuralMorphNetwork& UNeuralMorphNetworkInstance::GetNeuralNetwork() const
{ 
	return *Network.Get();
}

void UNeuralMorphNetworkInstance::Init(UNeuralMorphNetwork* InNeuralNetwork)
{
	Network = InNeuralNetwork;

	UE::NNE::IModelCPU* MainNeuralNetwork = Network->GetMainModel();
	UE::NNE::IModelCPU* GroupNeuralNetwork = Network->GetGroupModel();

	if (ensureMsgf(MainNeuralNetwork, TEXT("Main Neural Network must be valid to create instance")))
	{
		Inputs.SetNumZeroed(Network->GetNumInputs());   // This is just the number of inputs for the main network.
		Outputs.SetNumZeroed(Network->GetNumOutputs()); // This contains concatenated outputs of both main and group networks.
		MainInstance = MainNeuralNetwork->CreateModelInstanceCPU();
		MainInstance->SetInputTensorShapes({ UE::NNE::FTensorShape::Make({ 1, (uint32)Inputs.Num() }) });
	}

	if (GroupNeuralNetwork)
	{
		GroupInputs.SetNumZeroed(Network->GetNumGroupInputs());
		GroupInstance = GroupNeuralNetwork->CreateModelInstanceCPU();
		GroupInstance->SetInputTensorShapes({ UE::NNE::FTensorShape::Make({ 1, (uint32)GroupInputs.Num() }) });
	}
}

void UNeuralMorphNetworkInstance::Run()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UNeuralMorphNetwork::Run);

	UE::NNE::IModelCPU* MainNeuralNetwork = Network->GetMainModel();
	UE::NNE::IModelCPU* GroupNeuralNetwork = Network->GetGroupModel();

	TArrayView<float> MainOutputs = TArrayView<float>(Outputs.GetData(), Network->GetNumMainOutputs());

	if (Network->GetMode() == ENeuralMorphMode::Global)
	{
		checkfSlow(Network->GetNumFloatsPerCurve() == 1, TEXT("Expecting the number of floats per curve to be 1 in global mode."));

		MainInstance->RunSync(
			{ { (void*)Inputs.GetData(), Inputs.Num() * sizeof(float) } },
			{ { (void*)MainOutputs.GetData(), MainOutputs.Num() * sizeof(float) } });
	}
	else if (Network->GetMode() == ENeuralMorphMode::Local)
	{
		checkfSlow(Network->GetNumFloatsPerCurve() == 6, TEXT("Expecting num floats per curve to be 6 in local mode."));

		MainInstance->RunSync(
			{ { (void*)Inputs.GetData(), Inputs.Num() * sizeof(float) } },
			{ { (void*)MainOutputs.GetData(), MainOutputs.Num() * sizeof(float) } });

		if (GroupNeuralNetwork)
		{
			TArrayView<float> GroupOutputs = TArrayView<float>(
				Outputs.GetData() + Network->GetNumMainOutputs(),
				Network->GetNumGroupOutputs());

			GroupInstance->RunSync(
				{ { (void*)GroupInputs.GetData(), GroupInputs.Num() * sizeof(float) } },
				{ { (void*)GroupOutputs.GetData(), GroupOutputs.Num() * sizeof(float) } });
		}
	}
	else
	{
		checkf(false, TEXT("Unknown Mode"));
	}
}
