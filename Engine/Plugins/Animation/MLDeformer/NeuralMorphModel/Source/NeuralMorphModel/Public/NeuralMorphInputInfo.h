// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "MLDeformerMorphModelInputInfo.h"
#include "NeuralMorphTypes.h"
#include "MLDeformerModel.h"
#include "NeuralMorphInputInfo.generated.h"

#define UE_API NEURALMORPHMODEL_API

/**
 * The neural morph model's input info that contains all the data about the inputs to the network.
 */
UCLASS(MinimalAPI)
class UNeuralMorphInputInfo
	: public UMLDeformerMorphModelInputInfo
{
	GENERATED_BODY()

public:
	// FMLDeformerInputInfo overrides.
	UE_API virtual void Reset() override;
	// ~END FMLDeformerInputInfo overrides.

	/**
	 * Get the bone groups.
	 * Bone groups are groups of bones that generate morph targets all together as a group rather than one for each bone individually.
	 * @return A reference to the bone groups array.
	 */
	UE_API TArray<FNeuralMorphBoneGroup>& GetBoneGroups();
	UE_API const TArray<FNeuralMorphBoneGroup>& GetBoneGroups() const;

	/**
	 * Get the curve groups.
	 * Curve groups are groups of bones that generate morph targets all together as a group rather than one for each curve individually.
	 * @return A reference to the curve groups array.
	 */
	UE_API TArray<FNeuralMorphCurveGroup>& GetCurveGroups();
	UE_API const TArray<FNeuralMorphCurveGroup>& GetCurveGroups() const;

	/**
	 * Calculate the maximum number of group items.
	 * This will look in both the bone and curve groups.
	 * So for example, if there is a curve group with 5 items, while other groups have only 2 or 3 items, this method will return 5.
	 * @return The maximum item count of all groups.
	 */
	UE_API int32 CalcMaxNumGroupItems() const;

	/**
	 * Generate a list of index values that point inside the bone array.
	 * So the values that are output are either bone or curve numbers.
	 * The number indexes inside the array returned by GetBoneNames().
	 * The indices are a continuous array, with each padded group concatenated.
	 * If we have two groups, like: ['UpperArm_L', 'Clavicle_L', 'Spine3'], ['UpperArm_R', 'Clavicle_R'] then this method will output
	 * something like: [10, 11, 8, 15, 16, 16]. Where each number is the index inside our bone list.
	 * As you can see there is one value (number 16) that is repeated. This is because for every group we will output the same number of items.
	 * We find the maximum group item count, using CalcMaxGroupItems() and output number of values for each group.
	 * We simply repeat the last number one or more times if we have to pad the item count of that group. This is why the number 16 is repeated two times.
	 * @param OutBoneGroupIndices The array we will fill with the right bone indices. It will automatically resize internally.
	 */
	UE_API void GenerateBoneGroupIndices(TArray<int32>& OutBoneGroupIndices);

	/**
	 * Generates the list of index values that point inside the curve array.
	 * For more information please look at the GenerateBoneGroupIndices documentation.
	 * @param OutCurveGroupIndices The array we will fill with the right curve indices. It will automatically resize internally.
	 * @see GenerateBoneGroupIndices.
	 */
	UE_API void GenerateCurveGroupIndices(TArray<int32>& OutCurveGroupIndices);

	/**
	 * Check whether we have invalid groups or not.
	 * A group is considered invalid when it is empty, has items with the name 'None', or invalid names, or when the 
	 * items (bones or curve names) do not exist in the bone or curve list.
	 * @return Returns true when there are invalid bone or curve groups, otherwise false is returned.
	 */
	UE_API bool HasInvalidGroups() const;

	/**
	 * Get the skinning mode that we trained with.
	 * @return Returns the skinning mode, such as linear skinned or dual quaternion skinned.
	 */
	UE_API EMLDeformerSkinningMode GetSkinningMode() const;

	/**
	 * Set the skinning mode that was used to train with.
	 */
	UE_API void SetSkinningMode(EMLDeformerSkinningMode Mode);

protected:
	/** The groups of bones that generate morph targets together. */
	UPROPERTY()
	TArray<FNeuralMorphBoneGroup> BoneGroups;

	/** The groups of curves that generate morph targets together. */
	UPROPERTY()
	TArray<FNeuralMorphCurveGroup> CurveGroups;

	/** The skinning mode that was trained for. */
	UPROPERTY()
	EMLDeformerSkinningMode SkinningMode = EMLDeformerSkinningMode::Linear;
};

#undef UE_API
