// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "MLDeformerMorphModel.h"
#include "GeometryCache.h"
#include "NeuralMorphTypes.h"
#include "Containers/Map.h"
#include "NeuralMorphModel.generated.h"

#define UE_API NEURALMORPHMODEL_API

class USkeletalMesh;
class UGeometryCache;
class UAnimSequence;
class UNeuralMorphNetwork;
class UMLDeformerAsset;
class UMLDeformerModelInstance;
class USkeleton;
struct FExternalMorphSet;

// Declare our log category.
NEURALMORPHMODEL_API DECLARE_LOG_CATEGORY_EXTERN(LogNeuralMorphModel, Log, All);

/**
 * The neural morph model.
 * This generates a set of highly compressed morph targets to approximate a target deformation based on bone rotations and/or curve inputs.
 * During inference, the neural network inside this model runs on the CPU and outputs the morph target weights for the morph targets it generated during training.
 * Groups of bones and curves can also be defined. Those groups will generate a set of morph targets together as well. This can help when shapes depend on multiple inputs.
 * An external morph target set is generated during training and serialized inside the ML Deformer asset that contains this model.
 * When the ML Deformer component initializes the morph target set is registered. Most of the heavy lifting is done by the UMLDeformerMorphModel class.
 * The neural morph model has two modes: local and global. See the ENeuralMorphMode for a description of the two.
 * @see ENeuralMorphMode
 */
UCLASS(MinimalAPI)
class UNeuralMorphModel 
	: public UMLDeformerMorphModel
{
	GENERATED_BODY()

public:
	UE_API UNeuralMorphModel(const FObjectInitializer& ObjectInitializer);

	// UObject overrides.
	UE_API virtual void Serialize(FArchive& Archive) override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	// ~END UObject overrides.

	// UMLDeformerModel overrides.
	virtual FString GetDisplayName() const override					{ return "Neural Morph Model"; }
	UE_API virtual UMLDeformerModelInstance* CreateModelInstance(UMLDeformerComponent* Component) override;
	UE_API virtual UMLDeformerInputInfo* CreateInputInfo() override;
	UE_API virtual int32 GetNumFloatsPerCurve() const override;
	UE_API virtual bool IsTrained() const override;
	// ~END UMLDeformerModel overrides.

	/**
	 * Check if only global mode should be supported or not.
	 * Models that inherit from this model can disable this if they only want the global mode.
	 * This will automatically hide the UI elements related to the mode.
	 */
	virtual bool SupportsGlobalModeOnly() const						{ return false; }

	const TArray<FNeuralMorphBoneGroup>& GetBoneGroups() const		{ return BoneGroups; }
	const TArray<FNeuralMorphCurveGroup>& GetCurveGroups() const	{ return CurveGroups; }
	UNeuralMorphNetwork* GetNeuralMorphNetwork() const				{ return NeuralMorphNetwork.Get(); }
	ENeuralMorphMode GetModelMode() const							{ return SupportsGlobalModeOnly() ? ENeuralMorphMode::Global : Mode; }
	int32 GetLocalNumMorphsPerBone() const							{ return LocalNumMorphTargetsPerBone; }
	int32 GetLocalNumHiddenLayers() const							{ return LocalNumHiddenLayers; }
	int32 GetLocalNumNeuronsPerLayer() const						{ return LocalNumNeuronsPerLayer; }
	int32 GetGlobalNumMorphs() const								{ return GlobalNumMorphTargets; }
	int32 GetGlobalNumHiddenLayers() const							{ return GlobalNumHiddenLayers; }
	int32 GetGlobalNumNeuronsPerLayer() const						{ return GlobalNumNeuronsPerLayer; }
	int32 GetNumIterations() const									{ return NumIterations; }
	int32 GetBatchSize() const										{ return BatchSize; }
	int32 GetLocalNumMorphsPerBoneOrCurve() const					{ return LocalNumMorphTargetsPerBone; }
	float GetLearningRate() const									{ return LearningRate; }
	float GetSmoothLossBeta() const									{ return SmoothLossBeta; }
	float GetRegularizationFactor() const							{ return RegularizationFactor; }
	bool IsBoneMaskingEnabled() const								{ return bEnableBoneMasks; }

	EMLDeformerSkinningMode GetSkinningMode() const					{ return SkinningMode; }
	void SetSkinningMode(EMLDeformerSkinningMode SkinMode)			{ SkinningMode = SkinMode; }

	UE_API void UpdateMissingGroupNames();
	UE_API void SetNeuralMorphNetwork(UNeuralMorphNetwork* Net);
	void SetNumIterations(int32 InNumIterations)					{ check(InNumIterations > 0); NumIterations = InNumIterations; }

	static FName GetModePropertyName()								{ return GET_MEMBER_NAME_CHECKED(UNeuralMorphModel, Mode); }
	static FName GetSkinningModePropertyName()						{ return GET_MEMBER_NAME_CHECKED(UNeuralMorphModel, SkinningMode); }

public:
	/**
	 * The set of bones that are grouped together and generate morph targets together as a whole.
	 * This can be used in case multiple bones are correlated to each other and work together to produce given shapes.
	 * Groups are only used when the model is in Local mode.
	 */
	UPROPERTY()
	TArray<FNeuralMorphBoneGroup> BoneGroups;

	/**
	 * The set of curves that are grouped together and generate morph targets together as a whole.
	 * This can be used in case multiple curves are correlated to each other and work together to produce given shapes.
	 * Groups are only used when the model is in Local mode.
	 */
	UPROPERTY()
	TArray<FNeuralMorphCurveGroup> CurveGroups;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "This property has been deprecated, please use BoneMaskInfoMap instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property has been deprecated, please use BoneMaskInfoMap instead."))
	TMap<FName, FNeuralMorphMaskInfo> BoneMaskInfos_DEPRECATED;

	UE_DEPRECATED(5.5, "This property has been deprecated, please use BoneGroupMaskInfoMap instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property has been deprecated, please use BoneGroupMaskInfoMap instead."))
	TMap<FName, FNeuralMorphMaskInfo> BoneGroupMaskInfos_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Information needed to generate a mask for each bone.
	 * Each mask info object contains a list of bones who's skinning influence regions should be included in the final mask for the specific bone.
	 * This information (and bone masking in general) is used inside the ENeuralMorphMode::Local mode.
	 * The FName map key represents the bone name.
	 */
	UPROPERTY()
	TMap<FName, FMLDeformerMaskInfo> BoneMaskInfoMap;

	/**
	 * Information needed to generate a mask for each bone group.
	 * So the size of the array is NumGroupsInInputInfo.
	 * Each mask info object contains a list of bones who's skinning influence regions should be included in the final mask for the specific bone.
	 * This information (and bone masking in general) is used inside the ENeuralMorphMode::Local mode.
	 */
	UPROPERTY()
	TMap<FName, FMLDeformerMaskInfo> BoneGroupMaskInfoMap;

	/**
	 * The mode that the neural network will operate in. 
	 * Local mode means there is one tiny network per bone, while global mode has one network for all bones together.
	 * The advantage of local mode is that it has higher performance, while global mode might result in better deformations.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Training Settings")
	ENeuralMorphMode Mode = ENeuralMorphMode::Local;

	/** 
	 * The number of morph targets to generate per bone, curve or group.
	 * Higher numbers result in better approximation of the target deformation, but also result in a higher memory footprint and slower performance.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, DisplayName = "Num Morphs Per Bone/Curve/Group", Category = "Training Settings", meta = (ClampMin = "1", ClampMax = "1000"))
	int32 LocalNumMorphTargetsPerBone = 6;

	/** 
	 * The number of morph targets to generate in total. 
	 * Higher numbers result in better approximation of the target deformation, but also result in a higher memory footprint and slower performance.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, DisplayName = "Num Morph Targets", Category = "Training Settings", meta = (ClampMin = "1", ClampMax = "1000"))
	int32 GlobalNumMorphTargets = 128;

	/** 
	 * The number of iterations to train the model for. 
	 * If you are quickly iterating then around 1000 to 3000 iterations should be enough.
	 * If you want to generate final assets you might want to use a higher number of iterations, like 10k to 100k or even up to one million.
	 * Once the loss doesn't go down anymore, you know that more iterations most likely won't help much.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Training Settings", meta = (ClampMin = "1", ClampMax = "1000000"))
	int32 NumIterations = 5000;

	/** 
	 * The number of hidden layers that the neural network model will have.
	 * Higher numbers will slow down performance but can deal with more complex deformations. 
	 * For the local model you most likely want to stick with a value of one or two.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, DisplayName = "Num Hidden Layers", Category = "Training Settings", meta = (ClampMin = "0", ClampMax = "5"))
	int32 LocalNumHiddenLayers = 1;

	/** 
	 * The number of units/neurons per hidden layer. 
	 * Higher numbers will slow down performance but allow for more complex mesh deformations. 
	 * For the local mode you probably want to keep this around the same value as the number of morph targets per bone.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, DisplayName = "Num Neurons Per Layer", Category = "Training Settings", meta = (ClampMin = "1", ClampMax = "4096"))
	int32 LocalNumNeuronsPerLayer = 6;

	/** The number of hidden layers that the neural network model will have.\nHigher numbers will slow down performance but can deal with more complex deformations. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, DisplayName = "Num Hidden Layers", Category = "Training Settings", meta = (ClampMin = "0", ClampMax = "6"))
	int32 GlobalNumHiddenLayers = 2;

	/** The number of units/neurons per hidden layer. Higher numbers will slow down performance but allow for more complex mesh deformations. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, DisplayName = "Num Neurons Per Layer", Category = "Training Settings", meta = (ClampMin = "1", ClampMax = "4096"))
	int32 GlobalNumNeuronsPerLayer = 128;

	/** The number of frames per batch when training the model. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Training Settings", meta = (ClampMin = "1"))
	int32 BatchSize = 128;

	/** The learning rate used during the model training. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Training Settings", meta = (ClampMin = "0.000001", ClampMax = "1.0"))
	float LearningRate = 0.001f;

	/** 
	 * The regularization factor. Higher values can help generate more sparse morph targets, but can also lead to visual artifacts. 
	 * A value of 0 disables the regularization, and gives the highest quality, at the cost of higher runtime memory usage.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Training Settings", meta = (ClampMin = "0.0", ClampMax = "10.0"))
	float RegularizationFactor = 0.0f;

	/** 
	 * Enable the use of per bone and bone group masks.
	 * When enabled, an influence mask is generated per bone based on skinning info. This will enforce deformations localized to the area around the joint.
	 * The benefit of enabling this can be reduced GPU memory footprint, faster GPU performance and more localized deformations.
	 * If deformations do not happen near the joint then this enabling this setting can lead to those deformations possibly not being captured.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Training Settings", meta = (EditCondition = "Mode == ENeuralMorphMode::Local", EditConditionHides))
	bool bEnableBoneMasks = false;

	/**
	 * The beta parameter in the smooth L1 loss function, which describes below which absolute error to use a squared term. If the error is above or equal to this beta value, it will use the L1 loss.
	 * This is a non-negative value, where 0 makes it behave exactly the same as an L1 loss.
	 * The value entered represents how many cm of error to allow before an L1 loss is used. Typically higher values give smoother results.
	 * If you see some noise in the trained results, even with large amount of samples and iterations, try increasing this value.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Training Settings", meta = (ClampMin = "0.0", ClampMax = "10.0"))
	float SmoothLossBeta = 0.0f;

	/**
	 * The base skinning mode to use. The learned deltas will be applied on top of this basis.
	 * Please keep in mind that you use the same type of skinning on your character, otherwise the applied deltas are wrong.
	 * If you train using Dual Quaternions you should also run a Dual Quaternion skinning deformer before applying the ML Deltas on top.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Training Settings")
	EMLDeformerSkinningMode SkinningMode = EMLDeformerSkinningMode::Linear;

	/**
	 * The neural morph model network.
	 * This is a specialized network used for inference, for performance reasons.
	 */
	UPROPERTY()
	TObjectPtr<UNeuralMorphNetwork> NeuralMorphNetwork;
};

#undef UE_API
