// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MLDeformerMorphModelInstance.h"
#include "NeuralMorphModelInstance.generated.h"

#define UE_API NEURALMORPHMODEL_API

class UNeuralMorphNetworkInstance;
class UNeuralMorphNetwork;

UCLASS(MinimalAPI)
class UNeuralMorphModelInstance
	: public UMLDeformerMorphModelInstance
{
	GENERATED_BODY()

public:
	// UMLDeformerModelInstance overrides.
	UE_API virtual void Init(USkeletalMeshComponent* SkelMeshComponent) override;
	UE_API virtual int64 SetCurveValues(float* OutputBuffer, int64 OutputBufferSize, int64 StartIndex) override;
	UE_API virtual bool SetupInputs() override;
	UE_API virtual void Execute(float ModelWeight) override;
	// ~END UMLDeformerModelInstance overrides.

	UNeuralMorphNetworkInstance* GetNetworkInstance() const	{ return NetworkInstance.Get(); }

protected:
	/**
	 * Set the network inputs.
	 */
	UE_API void FillNetworkInputs();

protected:
	UPROPERTY(Transient)
	TObjectPtr<UNeuralMorphNetworkInstance> NetworkInstance;

	TArray<int32> BoneGroupIndices;
	TArray<int32> CurveGroupIndices;
};

#undef UE_API
