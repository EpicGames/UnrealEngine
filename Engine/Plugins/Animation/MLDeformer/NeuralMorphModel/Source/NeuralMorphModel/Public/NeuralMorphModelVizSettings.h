// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerMorphModelVizSettings.h"
#include "NeuralMorphTypes.h"
#include "NeuralMorphModel.h"
#include "NeuralMorphModelVizSettings.generated.h"

/**
 * The visualization settings specific to this model.
 * Even if we have no new properties compared to the morph model, we still need to
 * create this class in order to properly register a detail customization for it in our editor module.
 */
UCLASS(MinimalAPI)
class UNeuralMorphModelVizSettings
	: public UMLDeformerMorphModelVizSettings
{
	GENERATED_BODY()

public:
	UFUNCTION()
	bool ShouldShowMaskVizMode() const
	{
		return Cast<UNeuralMorphModel>(GetOuter())->GetModelMode() == ENeuralMorphMode::Local;
	}

	/**
	 * The visualization mode for the masks.
	 * Each bone, curve, bone group or curve group has a specific mask area on the mesh.
	 * This mask defines areas where generated morph targets can be active. They can be used to filter out deformations in undesired areas.
	 * For example if you rotate the left arm, you don't want the right arm to deform. The mask for the left arm can be setup in a way that it only includes 
	 * vertices around the area of the left arm to enforce this.
	 */
	UPROPERTY(EditAnywhere, Category = "Live Settings", meta = (EditCondition = "ShouldShowMaskVizMode()", EditConditionHides))
	ENeuralMorphMaskVizMode MaskVizMode = ENeuralMorphMaskVizMode::WhenInFocus;
};
