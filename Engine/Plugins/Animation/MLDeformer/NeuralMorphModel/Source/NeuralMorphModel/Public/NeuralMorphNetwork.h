// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "UObject/Object.h"
#include "NeuralMorphModel.h"
#include "NeuralMorphNetwork.generated.h"

#define UE_API NEURALMORPHMODEL_API

class UNNEModelData;

namespace UE::NNE
{
	class IModelCPU;
	class IModelInstanceCPU;
}

class UNeuralMorphNetworkInstance;

UCLASS()
class UE_DEPRECATED(5.4, "Neural Morph MLP no longer used for inference.") NEURALMORPHMODEL_API UNeuralMorphMLPLayer
	: public UObject
{
	GENERATED_BODY()

public:
	
	UPROPERTY()
	int32 NumInputs = 0;

	UPROPERTY()
	int32 NumOutputs = 0;

	UPROPERTY()
	int32 Depth = 1;

	UPROPERTY()
	TArray<float> Weights;

	UPROPERTY()
	TArray<float> Biases;
};


UCLASS()
class UE_DEPRECATED(5.4, "Neural Morph MLP longer used for inference.") NEURALMORPHMODEL_API UNeuralMorphMLP
	: public UObject
{
	GENERATED_BODY()

public:
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	void Empty();
	bool IsEmpty() const;
	bool Load(FArchive& FileReader);
	int32 GetNumInputs() const;
	int32 GetNumOutputs() const;
	int32 GetNumLayers() const;
	UNeuralMorphMLPLayer& GetLayer(int32 Index) const;
	int32 GetMaxNumLayerInputs() const;
	int32 GetMaxNumLayerOutputs() const;

private:
	
	UNeuralMorphMLPLayer* LoadNetworkLayer(FArchive& Archive);

private:

	UPROPERTY()
	TArray<TObjectPtr<UNeuralMorphMLPLayer>> Layers;

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

/**
 * The specialized neural network for the Neural Morph Model.
 * This class is used to do inference at runtime at a higher performance than using UNeuralNetwork.
 * The reason why it is faster is because it is a highly specialized network for this specific model.
 * When the model is a local mode based model, there can be actually two neural networks inside this: one main network and a bone/curve group network.
 * The group network uses groups of bones or curves to generate morph targets, rather than individual bones/curves.
 */
UCLASS(MinimalAPI)
class UNeuralMorphNetwork
	: public UObject
{
	GENERATED_BODY()

public:

	// These are required because of the use of TSharedPtr
	UE_API UNeuralMorphNetwork();
	UE_API UNeuralMorphNetwork(const FObjectInitializer& ObjectInitializer);
	UE_API UNeuralMorphNetwork(FVTableHelper& Helper);
	UE_API virtual ~UNeuralMorphNetwork();

	/** Used to convert the legacy neural network format to the NNE format */
	UE_API virtual void PostLoad() override;

	/** Clear the network, getting rid of all weights and biases. */
	UE_API void Empty();

	/** 
	 * Check if the network is empty or not.
	 * If it is empty, it means it hasn't been loaded, and cannot do anything.
	 * Empty means it has no inputs, outputs or weights.
	 * @return Returns true if empty, otherwise false is returned.
	 */
	UE_API bool IsEmpty() const;

	/**
	 * Load the network from a file on disk.
	 * When loading fails, the network will be emptied.
	 * @param Filename The file path to load from.
	 * @return Returns true when successfully loaded this model, otherwise false is returned.
	 */
	UE_API bool Load(const FString& Filename);

	/**
	 * Create an instance of this neural network.
	 * @return A pointer to the neural network instance.
	 */
	UE_API UNeuralMorphNetworkInstance* CreateInstance();

	/**
	 * Get the number of network inputs for the main network.
	 * @return The number of inputs.
	 */
	UE_API int32 GetNumMainInputs() const;

	/**
	 * Get the number of network outputs for the main network.
	 * @return The number of outputs.
	 */
	UE_API int32 GetNumMainOutputs() const;

	/**
	 * Get the number of network inputs for the group network.
	 * @return The number of inputs.
	 */
	UE_API int32 GetNumGroupInputs() const;

	/**
	 * Get the number of network outputs for the group network.
	 * @return The number of outputs.
	 */
	UE_API int32 GetNumGroupOutputs() const;

	/**
	 * Get the number of network inputs.
	 * This is the number of main network inputs.
	 * @return The number of inputs.
	 */
	UE_API int32 GetNumInputs() const;

	/**
	 * Get the number of network outputs.
	 * This is the sum of the number of main network outputs and group network outputs.
	 * @return The number of outputs.
	 */
	UE_API int32 GetNumOutputs() const;

	/**
	 * Get the number of bones that are input to the network.
	 * @return The number of bones.
	 */
	UE_API int32 GetNumBones() const;

	/**
	 * Get the number of curves that are input to the network.
	 * @return The number of curves.
	 */
	UE_API int32 GetNumCurves() const;

	/**
	 * Get the number of morph targets per bone. This will only be valid if the GetMode() method returns the local model mode.
	 * For global mode networks, this will return 0.
	 * @return The number of morph targets per bone.
	 */
	UE_API int32 GetNumMorphsPerBone() const;

	/**
	 * Get the number of groups.
	 * A group is a collection of bones or curves that together generate a number of morph targets.
	 * Each group must have the same number of items.
	 * @result The number of groups.
	 */
	UE_API int32 GetNumGroups() const;

	/**
	 * Get the number of items per group.
	 * This indicates how many bones or curves each group has.
	 * Every group must have the same number of items.
	 * @return The number of items per group.
	 */
	UE_API int32 GetNumItemsPerGroup() const;

	/**
	 * Get the mode that the model was trained for, either global or local mode.
	 * @return The mode the model was trained for.
	 */
	UE_API ENeuralMorphMode GetMode() const;

	/**
	 * Get the means of each input, used for normalizing the input values.
	 * @return The array of mean values, one value for each network input.
	 */
	UE_API const TArrayView<const float> GetInputMeans() const;

	/**
	 * Get the standard deviations of each input, used for normalizing the input values.
	 * @return The array that contains the standard deviations, one value for each network input.
	 */
	UE_API const TArrayView<const float> GetInputStds() const;

	/** 
	 * Get the number of floats used to represent a single curve value.
	 * @return The number of float values per curve.
	 */
	UE_API int32 GetNumFloatsPerCurve() const;

	/**
	 * Get the main Model.
	 * This is the Model (or actually a Multi-Model) that contains a small network for every bone or curve.
	 * Each small network will output a set of morph target weights.
	 * @return A pointer to the main Model.
	 */
	UE_API UE::NNE::IModelCPU* GetMainModel() const;

	/**
	 * Get the Model used for group of bones and curves.
	 * This Model (or actually a Multi-Model) contains small networks for the bone or curve groups.
	 * Each group will generate a set of morph target weights.
	 * @return A pointer to the Model used for the bone and curve groups.
	 */
	UE_API UE::NNE::IModelCPU* GetGroupModel() const;

private:

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.4, "MLP Object no longer available. Please use MainModel instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Network format changed. Please use MainModel instead."))
	TObjectPtr<UNeuralMorphMLP> MainMLP_DEPRECATED;

	UE_DEPRECATED(5.4, "MLP Object no longer available. Please use GroupModel instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Network format changed. Please use GroupModel instead."))
	TObjectPtr<UNeuralMorphMLP> GroupMLP_DEPRECATED;

	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** The Model Data for the Main Network */
	UPROPERTY()
	TObjectPtr<UNNEModelData> MainModelData;

	/** The Neural Network that acts as main network. */
	TSharedPtr<UE::NNE::IModelCPU> MainModel;

	/** The Model Data for the Group Network */
	UPROPERTY()
	TObjectPtr<UNNEModelData> GroupModelData;

	/** The Neural Network for the bone and curve groups, when in local mode. */
	TSharedPtr<UE::NNE::IModelCPU> GroupModel;

	/** The means of the input values, used to normalize inputs. */
	UPROPERTY()
	TArray<float> InputMeans;

	/** The standard deviation of the input values, used to normalize inputs. */
	UPROPERTY()
	TArray<float> InputStd;

	/** The mode of the network, either local or global. */
	UPROPERTY()
	ENeuralMorphMode Mode = ENeuralMorphMode::Global;

	/** The total number of morph targets, if set Mode == Global, otherwise ignored. */
	UPROPERTY()
	int32 NumMorphs = 0;

	/** The number of morph targets per bone, if set Mode == Local, otherwise ignored. */
	UPROPERTY()
	int32 NumMorphsPerBone = 0;

	/** The number of bones that were input. */
	UPROPERTY()
	int32 NumBones = 0;

	/** The number of curves that were input. */
	UPROPERTY()
	int32 NumCurves = 0;

	/** The number of floats per curve. */
	UPROPERTY()
	int32 NumFloatsPerCurve = 1;

	/** The number of groups. This is a group of bones/curves that create their own morphs. */
	UPROPERTY()
	int32 NumGroups = 0;

	/** The number of items (bones/curves) per group. */
	UPROPERTY()
	int32 NumItemsPerGroup = 0;
};

/** 
 * An instance of a UNeuralMorphNetwork.
 * The instance holds its own input and output buffers and only reads from the network object it was instanced from.
 * This allows it to be multithreaded.
 */
UCLASS(MinimalAPI)
class UNeuralMorphNetworkInstance
	: public UObject
{
	friend class UNeuralMorphNetwork;

	GENERATED_BODY()

public:

	// These are required because of the use of TSharedPtr
	UE_API UNeuralMorphNetworkInstance();
	UE_API UNeuralMorphNetworkInstance(const FObjectInitializer& ObjectInitializer);
	UE_API UNeuralMorphNetworkInstance(FVTableHelper& Helper);
	UE_API virtual ~UNeuralMorphNetworkInstance();

	/**
	 * Get the network input buffer.
	 * @return The array view of floats that represents the main network inputs.
	 */
	UE_API TArrayView<float> GetInputs();
	UE_API TArrayView<const float> GetInputs() const;

	/**
	 * Get the inputs for the group MLP.
	 * @return The array view of floats that represent the group network inputs.
	 */
	UE_API TArrayView<float> GetGroupInputs();
	UE_API TArrayView<const float> GetGroupInputs() const;

	/**
	 * Get the network output buffer.
	 * @return The array view of floats that represents the network outputs.
	 */
	UE_API TArrayView<float> GetOutputs();
	UE_API TArrayView<const float> GetOutputs() const;

	/**
	 * Get the neural network this is an instance of.
	 * @return A reference to the neural network.
	 */
	UE_API const UNeuralMorphNetwork& GetNeuralNetwork() const;

	/**
	 * Run the neural network, performing inference.
	 * This will update the values in the output buffer that you can get with the GetOutputs method.
	 * This also assumes you have set all the right input values already.
	 */
	UE_API void Run();

private:

	/**
 	 * Initialize this instance using a given network object.
	 * This is automatically called by the UNeuralMorphNetwork::CreateInstance() method.
	 * @param InNeuralNetwork The network that this is an instance of.
	 */
	UE_API void Init(UNeuralMorphNetwork* InNeuralNetwork);

private:
	/** The input values for the main network. */
	TArray<float> Inputs;

	/** The input values for the group network. */
	TArray<float> GroupInputs;
	
	/** The output values. */
	TArray<float> Outputs;

	/** The inference instance data for the main Neural Network */
	TSharedPtr<UE::NNE::IModelInstanceCPU> MainInstance;

	/** The inference instance data for the group Neural Network */
	TSharedPtr<UE::NNE::IModelInstanceCPU> GroupInstance;

	/** The neural network this is an instance of. */
	UPROPERTY(Transient)
	TObjectPtr<UNeuralMorphNetwork> Network;
};

#undef UE_API
