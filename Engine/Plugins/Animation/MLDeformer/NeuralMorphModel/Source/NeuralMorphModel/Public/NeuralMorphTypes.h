// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "BoneContainer.h"
#include "MLDeformerMasking.h"
#include "MLDeformerCurveReference.h"
#include "NeuralMorphTypes.generated.h"

/** 
 * The mode of the model, either local or global. 
 * In local mode the network contains a super simple neural network for each bone, while in 
 * global mode all bones and curves are input to one larger fully connected network.
 * The local mode has faster performance, while global mode can result in higher quality deformation.
 * This model runs its neural network on the CPU, but uses comrpessed GPU based morph targets, which require shader model 5.
 */
UENUM()
enum class ENeuralMorphMode : uint8
{
	/**
	 * Each bone creates a set of morph targets and has its own small neural network.
	 * The local mode can also create more localized morph targets and tends to use slightly less memory.
	 * This mode is faster to process on the CPU side.
	 */
	Local,

	/** 
	 * There is one fully connected neural network that generates a set of morph targets.
	 * This has a slightly higher CPU overhead, but could result in higher quality.
	 * The Global mode is basically the same as the Vertex Delta Model, but runs the neural network on the CPU
	 * and uses GPU compressed morph targets.
	 */
	Global
};

/**
 * The visualization mode for the masks.
 * Each bone, curve, bone group or curve group has a specific mask area on the mesh.
 * This mask defines areas where generated morph targets can be active. They can be used to filter out deformations in undesired areas.
 * For example if you rotate the left arm, you don't want the right arm to deform. The mask for the left arm can be setup in a way that it only includes 
 * vertices around the area of the left arm to enforce this.
 */
UENUM()
enum class ENeuralMorphMaskVizMode : uint8
{
	/** Do not display the masks in the viewport. */
	Off,

	/** Only show the masks inside the viewport when the inputs widget on the right side of the UI is in focus. Show the mask for the selected item (bone, curve, bone group, curve group). */
	WhenInFocus,

	/** Always show the selected mask. The mask selected is defined by what is selected in the input widget on the right side of the UI. */
	Always
};

/**
 * A group of bones, which can generate morph targets together.
 * This is useful when there are specific correlations between different bones, that all effect the same region on the body.
 */
USTRUCT()
struct FNeuralMorphBoneGroup
{
	GENERATED_BODY()

public:
	/** The name of the group, also shown in the UI. */
	UPROPERTY()
	FName GroupName;

	/** The list of bones that should form a group together. */
	UPROPERTY(EditAnywhere, Category = "Bones")
	TArray<FBoneReference> BoneNames;
};

/**
 * A group of curves, which can generate morph targets together.
 * This is useful when there are specific correlations between different curves, that all effect the same region on the body.
 */
USTRUCT()
struct FNeuralMorphCurveGroup
{
	GENERATED_BODY()

public:
	/** The name of the group, also shown in the UI. */
	UPROPERTY()
	FName GroupName;

	/** The list of curves that should form a group together. */
	UPROPERTY(EditAnywhere, Category = "Curves")
	TArray<FMLDeformerCurveReference> CurveNames;
};


/**
 * Information needed to generate the mask for a specific bone.
 * This includes a list of bone names. Each bone has a skinning influence mask.
 * The final mask will be a merge of all the bone masks of bones listed inside this info struct.
 * There will be an array of these structs, one for each bone.
 */
USTRUCT()
struct UE_DEPRECATED(5.5, "This class has been deprecated, please use FMLDeformerMaskInfo instead.") NEURALMORPHMODEL_API FNeuralMorphMaskInfo
{
	GENERATED_BODY()

public:
	/** The list of bone names that should be included in the mask generation. */
	UPROPERTY()
	TArray<FName> BoneNames;

	/** The masking mode. */
	UPROPERTY()
	EMLDeformerMaskingMode MaskMode = EMLDeformerMaskingMode::Generated;

	/** If the masking mode is set to VertexAttribute then we can check which attribute to use using this member. */
	UPROPERTY()
	FName VertexAttributeName;
};
