// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class NeuralMorphModelEditor : ModuleRules
	{
		public NeuralMorphModelEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"InputCore",
					"UnrealEd",
					"CoreUObject",
					"Engine",
					"Slate",
					"SlateCore",
					"MLDeformerFramework",
					"MLDeformerFrameworkEditor",
					"NeuralMorphModel",
					"PropertyEditor",
					"ToolWidgets",
					"DeveloperSettings",
					"Json",
					"ApplicationCore"
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
				}
			);
		}
	}
}
