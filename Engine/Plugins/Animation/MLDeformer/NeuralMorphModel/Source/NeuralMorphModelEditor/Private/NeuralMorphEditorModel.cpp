// Copyright Epic Games, Inc. All Rights Reserved.

#include "NeuralMorphEditorModel.h"
#include "IDetailsView.h"
#include "NeuralMorphModel.h"
#include "NeuralMorphNetwork.h"
#include "NeuralMorphInputInfo.h"
#include "NeuralMorphTrainingModel.h"
#include "NeuralMorphModelVizSettings.h"
#include "NeuralMorphEditorProjectSettings.h"
#include "MLDeformerMorphModelVizSettings.h"
#include "MLDeformerEditorToolkit.h"
#include "MLDeformerEditorActor.h"
#include "SMLDeformerInputWidget.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/Skeleton.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "BoneContainer.h"
#include "BoneWeights.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "ReferenceSkeleton.h"
#include "Misc/TransactionObjectEvent.h"
#include "SNeuralMorphBoneGroupsWidget.h"
#include "SNeuralMorphCurveGroupsWidget.h"
#include "SNeuralMorphInputWidget.h"

#define LOCTEXT_NAMESPACE "NeuralMorphEditorModel"

namespace UE::NeuralMorphModel
{
	using namespace UE::MLDeformer;

	FMLDeformerEditorModel* FNeuralMorphEditorModel::MakeInstance()
	{
		return new FNeuralMorphEditorModel();
	}

	void FNeuralMorphEditorModel::RebuildEditorMaskInfo()
	{
		UNeuralMorphInputInfo* NeuralInputInfo = Cast<UNeuralMorphInputInfo>(GetEditorInputInfo());
		if (NeuralInputInfo)
		{
			RemoveNonExistingMaskInfos();
			UpdateEditorInputInfo();
			BuildMaskBuffer(NeuralInputInfo->GetInputItemMaskBuffer());
		}
	}

	void FNeuralMorphEditorModel::OnPropertyChanged(FPropertyChangedEvent& PropertyChangedEvent)
	{
		const FProperty* Property = PropertyChangedEvent.Property;
		if (Property == nullptr)
		{
			return;
		}

		// Process the base class property changes.
		FMLDeformerMorphModelEditorModel::OnPropertyChanged(PropertyChangedEvent);

		if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UNeuralMorphModel, BoneGroups) ||
			Property->GetFName() == GET_MEMBER_NAME_CHECKED(UNeuralMorphModel, CurveGroups) ||
			Property->GetFName() == TEXT("BoneNames") ||	// FNeuralMorphBoneGroup::BoneNames has items added or removed (it is an array).
			Property->GetFName() == TEXT("CurveNames") ||	// FNeuralMorphCurveGroup::CurveNames has items added or removed (it is an array).
			Property->GetFName() == TEXT("BoneName") ||		// The bone name inside one of the items in the BoneNames list changed.
			Property->GetFName() == TEXT("CurveName"))		// The curve name inside one of the items in the CurveNames list changed.
		{
			UpdateIsReadyForTrainingState();
			RebuildEditorMaskInfo();
		}
		else if (Property->GetFName() == UNeuralMorphModel::GetModePropertyName())
		{
			if (PropertyChangedEvent.ChangeType & EPropertyChangeType::ValueSet || PropertyChangedEvent.ChangeType & EPropertyChangeType::ResetToDefault)
			{
				SetResamplingInputOutputsNeeded(true);
				UpdateIsReadyForTrainingState();
				GetEditor()->GetModelDetailsView()->ForceRefresh();
				RebuildEditorMaskInfo();
			}
		}
		else if (Property->GetFName() == UNeuralMorphModel::GetSkinningModePropertyName())
		{	
			SetResamplingInputOutputsNeeded(true);
			for (TSharedPtr<FMLDeformerSampler> Sampler : Samplers)
			{
				if (Sampler.IsValid())
				{
					Sampler->SetSkinningMode(GetNeuralMorphModel()->GetSkinningMode());
				}
			}
		}
	}

	TSharedPtr<SMLDeformerInputWidget> FNeuralMorphEditorModel::CreateInputWidget()
	{
		return SNew(SNeuralMorphInputWidget)
			.EditorModel(this);
	}

	void FNeuralMorphEditorModel::UpdateTrainingDeviceList()
	{
		// Update the list of devices we can use to train with.
		// This is the cpu and list of cuda devices.
		UNeuralMorphTrainingModel* TrainingModel = NewDerivedObject<UNeuralMorphTrainingModel>();
		if (TrainingModel)
		{
			TrainingModel->Init(this);
			TrainingModel->UpdateAvailableDevices();
			TrainingModel->ConditionalBeginDestroy();
		}
	}

	UMeshDeformer* FNeuralMorphEditorModel::GetActiveHeatMapDeformer() const
	{
		if (Model->IsTrained())
		{
			UNeuralMorphInputInfo* NeuralMorphInputInfo = Cast<UNeuralMorphInputInfo>(Model->GetInputInfo());
			check(NeuralMorphInputInfo);

			const EMLDeformerSkinningMode SkinningMode = NeuralMorphInputInfo->GetSkinningMode();
			if (SkinningMode == EMLDeformerSkinningMode::DualQuaternion)
			{
				return HeatMapDeformerGraphDualQuat;
			}
		}

		// Default linear skinned heatmap.
		return HeatMapDeformerGraph;
	}

	void FNeuralMorphEditorModel::Init(const InitSettings& Settings)
	{
		FMLDeformerMorphModelEditorModel::Init(Settings);
		RemoveNonExistingMaskInfos();
	}

	ETrainingResult FNeuralMorphEditorModel::Train()
	{
		return TrainModel<UNeuralMorphTrainingModel>(this);
	}

	bool FNeuralMorphEditorModel::LoadTrainedNetwork() const
	{
		// Load the specialized neural morph model network.
		// Base the filename on the onnx filename, and replace the file extension.
		FString NetworkFilename = GetTrainedNetworkOnnxFile();
		NetworkFilename.RemoveFromEnd(TEXT("onnx"));
		NetworkFilename += TEXT("nmn");

		// Load the actual network.
		UNeuralMorphNetwork* NeuralNet = NewObject<UNeuralMorphNetwork>(Model);
		if (!NeuralNet->Load(NetworkFilename))
		{
			UE_LOGF(LogNeuralMorphModel, Error, "Failed to load neural morph network from file '%ls'!", *NetworkFilename);
			NeuralNet->ConditionalBeginDestroy();

			// Restore the deltas to the ones before training started.
			GetMorphModel()->SetMorphTargetDeltas(MorphTargetDeltasBackup);
			return false;
		}

		if (NeuralNet->IsEmpty())
		{
			NeuralNet->ConditionalBeginDestroy();
			NeuralNet = nullptr;
		}

		GetNeuralMorphModel()->SetNeuralMorphNetwork(NeuralNet);	// Use our custom inference.
		return true;
	}

	void FNeuralMorphEditorModel::InitInputInfo(UMLDeformerInputInfo* InputInfo)
	{
		FMLDeformerEditorModel::InitInputInfo(InputInfo);

		UNeuralMorphInputInfo* NeuralInputInfo = Cast<UNeuralMorphInputInfo>(InputInfo);
		if (NeuralInputInfo == nullptr)
		{
			return;
		}

		UNeuralMorphModel* NeuralMorphModel = Cast<UNeuralMorphModel>(Model);
		NeuralInputInfo->SetSkinningMode(NeuralMorphModel->GetSkinningMode());

		if (NeuralMorphModel->GetModelMode() != ENeuralMorphMode::Local)
		{
			return;
		}

		const USkeletalMesh* SkeletalMesh = Model->GetSkeletalMesh();

		// Handle bones.
		if (SkeletalMesh)
		{
			const TArray<FNeuralMorphBoneGroup>& ModelBoneGroups = NeuralMorphModel->GetBoneGroups();
			const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
			for (int32 BoneGroupIndex = 0; BoneGroupIndex < ModelBoneGroups.Num(); ++BoneGroupIndex)
			{
				const FNeuralMorphBoneGroup& BoneGroup = ModelBoneGroups[BoneGroupIndex];

				NeuralInputInfo->GetBoneGroups().AddDefaulted();
				FNeuralMorphBoneGroup& NewGroup = NeuralInputInfo->GetBoneGroups().Last();

				const int32 NumBonesInGroup = BoneGroup.BoneNames.Num();
				NewGroup.BoneNames.AddDefaulted(NumBonesInGroup);
				NewGroup.GroupName = BoneGroup.GroupName;
				for (int32 Index = 0; Index < NumBonesInGroup; ++Index)
				{
					const FBoneReference& BoneRef = BoneGroup.BoneNames[Index];
					if (!BoneRef.BoneName.IsValid() || BoneRef.BoneName.IsNone())
					{
						UE_LOGF(LogNeuralMorphModel, Warning, "Invalid or 'None' bone detected inside curve group %d, please fix this.", BoneGroupIndex);
						continue;
					}

					if (RefSkeleton.FindBoneIndex(BoneRef.BoneName) == INDEX_NONE)
					{
						UE_LOGF(LogNeuralMorphModel, Warning, "Bone '%ls' inside bone group %d doesn't exist, please fix this.", *BoneRef.BoneName.ToString(), BoneGroupIndex);
						continue;
					}

					if (!InputInfo->GetBoneNames().Contains(BoneRef.BoneName))
					{
						UE_LOGF(LogNeuralMorphModel, Warning, "Bone '%ls' inside bone group %d isn't included in the bone list that are input to the model.", *BoneRef.BoneName.ToString(), BoneGroupIndex);
						continue;
					}

					NewGroup.BoneNames[Index] = BoneRef.BoneName;
				}
			}
		}

		// Handle curves.
		if (SkeletalMesh)
		{
			const TArray<FNeuralMorphCurveGroup>& ModelCurveGroups = NeuralMorphModel->GetCurveGroups();
			for (int32 CurveGroupIndex = 0; CurveGroupIndex < ModelCurveGroups.Num(); ++CurveGroupIndex)
			{				
				const FNeuralMorphCurveGroup& CurveGroup = ModelCurveGroups[CurveGroupIndex];
				if (CurveGroup.CurveNames.IsEmpty())
				{
					continue;
				}

				NeuralInputInfo->GetCurveGroups().AddDefaulted();
				FNeuralMorphCurveGroup& NewGroup = NeuralInputInfo->GetCurveGroups().Last();

				const int32 NumCurvesInGroup = CurveGroup.CurveNames.Num();
				NewGroup.CurveNames.AddDefaulted(NumCurvesInGroup);
				NewGroup.GroupName = CurveGroup.GroupName;
				for (int32 Index = 0; Index < NumCurvesInGroup; ++Index)
				{
					const FMLDeformerCurveReference& CurveRef = CurveGroup.CurveNames[Index];
					if (!CurveRef.CurveName.IsValid() || CurveRef.CurveName.IsNone())
					{
						UE_LOGF(LogNeuralMorphModel, Warning, "Invalid or 'None' curve detected inside curve group %d, please fix this.", CurveGroupIndex);
						continue;
					}

					if (!InputInfo->GetCurveNames().Contains(CurveRef.CurveName))
					{
						UE_LOGF(LogNeuralMorphModel, Warning, "Curve '%ls' inside curve group %d isn't included in the curve list that are input to the model.", *CurveRef.CurveName.ToString(), CurveGroupIndex);
						continue;
					}

					NewGroup.CurveNames[Index] = CurveRef.CurveName;
				}
			}
		}

		RemoveNonExistingMaskInfos();
		//BuildMaskBuffer(NeuralInputInfo->GetInputItemMaskBuffer());
	}

	void FNeuralMorphEditorModel::UpdateIsReadyForTrainingState()
	{
		FMLDeformerMorphModelEditorModel::UpdateIsReadyForTrainingState();
		UNeuralMorphInputInfo* Info = Cast<UNeuralMorphInputInfo>(GetEditorInputInfo());
		if (Info)
		{
			if (GetNeuralMorphModel()->GetModelMode() == ENeuralMorphMode::Local)
			{
				bIsReadyForTraining &= !Info->HasInvalidGroups();
			}
		}
	}

	FText FNeuralMorphEditorModel::GetOverlayText() const
	{
		FText Text = FMLDeformerMorphModelEditorModel::GetOverlayText();
		UNeuralMorphInputInfo* Info = Cast<UNeuralMorphInputInfo>(GetEditorInputInfo());
		if (GetNeuralMorphModel()->GetModelMode() == ENeuralMorphMode::Local && Info->HasInvalidGroups())
		{
			Text = FText::Format(
				LOCTEXT("GroupErrorFormat", "{0}\n{1}"), 
				Text,
				LOCTEXT("GroupErrorText", "There are invalid bone and/or curve groups.\nCheck the log warnings for more information."));
		}
		return Text;
	}

	const TArrayView<const float> FNeuralMorphEditorModel::GetMaskForMorphTarget(int32 MorphTargetIndex) const
	{
		const UNeuralMorphModel* NeuralMorphModel = GetNeuralMorphModel();
		const UMLDeformerMorphModelInputInfo* MorphInputInfo = Cast<UMLDeformerMorphModelInputInfo>(NeuralMorphModel->GetInputInfo());
		check(MorphInputInfo);

		const TArray<float>& MaskBuffer = MorphInputInfo->GetInputItemMaskBuffer();
		if (MaskBuffer.IsEmpty() || !NeuralMorphModel->IsBoneMaskingEnabled() || NeuralMorphModel->GetModelMode() != ENeuralMorphMode::Local)
		{
			return TArrayView<const float>();
		}

		const int32 ItemIndex = MorphTargetIndex / NeuralMorphModel->GetLocalNumMorphsPerBone();
		return MorphInputInfo->GetMaskForItem(ItemIndex);
	}

	void FNeuralMorphEditorModel::BuildMaskBuffer(TArray<float>& OutMaskBuffer)
	{
		if (Model->GetSkeletalMesh() == nullptr)
		{
			OutMaskBuffer.Empty();
			return;
		}

		// Clear the mask buffer.
		UNeuralMorphModel* NeuralMorphModel = GetNeuralMorphModel();
		OutMaskBuffer.Empty();
		if (NeuralMorphModel->GetModelMode() != ENeuralMorphMode::Local)
		{
			return;
		}
		
		const int32 NumBaseMeshVerts = Model->GetNumBaseMeshVerts();

		USkeletalMesh* SkeletalMesh = Model->GetSkeletalMesh();	
		UNeuralMorphInputInfo* NeuralMorphInputInfo = Cast<UNeuralMorphInputInfo>(GetEditorInputInfo());
		const TArray<FName>& BoneNames = NeuralMorphInputInfo->GetBoneNames();

		// Calculate the number of flo =ats we need to store all the masks.
		const int32 NumBones = NeuralMorphInputInfo->GetNumBones();
		const int32 NumCurves = NeuralMorphInputInfo->GetNumCurves();
		const int32 NumBoneGroups = NeuralMorphInputInfo->GetBoneGroups().Num();
		const int32 NumCurveGroups = NeuralMorphInputInfo->GetCurveGroups().Num();
		const int32 NumFloats = (NumBones + NumCurves + NumBoneGroups + NumCurveGroups) * NumBaseMeshVerts;
		constexpr int32 HierarchyDepth = 1;	// Default hierarchy depth in case we have no mask for specific bones yet.

		// Init the mask to all zeros.
		OutMaskBuffer.SetNumZeroed(NumFloats);

		// For all bones.
		int32 MaskOffset = 0;
		for (int32 Index = 0; Index < NumBones; ++Index)
		{
			TArrayView<float> ItemMaskBuffer(&OutMaskBuffer[MaskOffset], NumBaseMeshVerts);
			const FName BoneName = BoneNames[Index];
			FMLDeformerMaskInfo* MaskInfo = NeuralMorphModel->BoneMaskInfoMap.Find(BoneName);
			if (MaskInfo == nullptr)
			{
				MaskInfo = &NeuralMorphModel->BoneMaskInfoMap.Add(BoneName, FMLDeformerMaskInfo());
				GenerateBoneMaskInfo(Index, HierarchyDepth);
			}

			ApplyMaskInfoToBuffer(SkeletalMesh, *MaskInfo, ItemMaskBuffer);
			MaskOffset += NumBaseMeshVerts;
		}

		// For all curves, just use a mask full of values of 1.
		for (int32 Index = 0; Index < NumCurves; ++Index)
		{
			TArrayView<float> ItemMaskBuffer(&OutMaskBuffer[MaskOffset], NumBaseMeshVerts);
			FillMaskValues(ItemMaskBuffer, 1.0f);
			MaskOffset += NumBaseMeshVerts;
		}

		// For all bone groups.		
		for (int32 Index = 0; Index < NumBoneGroups; ++Index)
		{
			TArrayView<float> ItemMaskBuffer(&OutMaskBuffer[MaskOffset], NumBaseMeshVerts);
			
			const FName GroupName = NeuralMorphInputInfo->GetBoneGroups()[Index].GroupName;
			FMLDeformerMaskInfo* MaskInfo = NeuralMorphModel->BoneGroupMaskInfoMap.Find(GroupName);
			if (MaskInfo == nullptr)
			{
				MaskInfo = &NeuralMorphModel->BoneGroupMaskInfoMap.Add(GroupName, FMLDeformerMaskInfo());
				GenerateBoneGroupMaskInfo(Index, HierarchyDepth);
			}

			ApplyMaskInfoToBuffer(SkeletalMesh, *MaskInfo, ItemMaskBuffer);
			MaskOffset += NumBaseMeshVerts;
		}

		// For all curve groups, just use a mask full of values of 1.
		for (int32 Index = 0; Index < NumCurveGroups; ++Index)
		{
			TArrayView<float> ItemMaskBuffer(&OutMaskBuffer[MaskOffset], NumBaseMeshVerts);
			FillMaskValues(ItemMaskBuffer, 1.0f);
			MaskOffset += NumBaseMeshVerts;
		}
	}

	FName FNeuralMorphEditorModel::GenerateUniqueBoneGroupName() const
	{
		const UNeuralMorphModel* NeuralMorphModel = GetNeuralMorphModel();
		FName GeneratedName;
		bool bUniqueNameFound = false;
		int32 Counter = 0;
		while (!bUniqueNameFound)
		{
			GeneratedName = FName(FString::Format(TEXT("Bone Group #{0}"), {Counter++}));

			bool bHasGroupWithSameName = false;
			for (const FNeuralMorphBoneGroup& BoneGroup : NeuralMorphModel->GetBoneGroups())
			{
				if (BoneGroup.GroupName == GeneratedName)
				{
					bHasGroupWithSameName = true;
					break;
				}
			}

			if (!bHasGroupWithSameName)
			{
				bUniqueNameFound = true;
			}
		}

		return GeneratedName;
	}

	FName FNeuralMorphEditorModel::GenerateUniqueCurveGroupName() const
	{
		const UNeuralMorphModel* NeuralMorphModel = GetNeuralMorphModel();
		FName GeneratedName;
		bool bUniqueNameFound = false;
		int32 Counter = 0;
		while (!bUniqueNameFound)
		{
			GeneratedName = FName(FString::Format(TEXT("Curve Group #{0}"), {Counter++}));

			bool bHasGroupWithSameName = false;
			for (const FNeuralMorphCurveGroup& CurveGroup : NeuralMorphModel->GetCurveGroups())
			{
				if (CurveGroup.GroupName == GeneratedName)
				{
					bHasGroupWithSameName = true;
					break;
				}
			}

			if (!bHasGroupWithSameName)
			{
				bUniqueNameFound = true;
			}
		}

		return GeneratedName;
	}

	void FNeuralMorphEditorModel::ResetBoneMaskInfos()
	{
		GetNeuralMorphModel()->BoneMaskInfoMap.Empty();
	}

	void FNeuralMorphEditorModel::ResetBoneGroupMaskInfos()
	{
		GetNeuralMorphModel()->BoneGroupMaskInfoMap.Empty();
	}

	void FNeuralMorphEditorModel::AddTwistBones(const FReferenceSkeleton& RefSkel, TArray<int32>& SkelBoneIndices)
	{
		check(UNeuralMorphEditorProjectSettings::Get());

		// Get the twist substring from the per-project configuration.
		// If we have an empty string, just skip the whole twist bone handling.
		const FString& TwistSubString = UNeuralMorphEditorProjectSettings::Get()->TwistBoneFilter;
		if (TwistSubString.IsEmpty())
		{
			return;
		}

		// Add child bones with "twist" in their name.
		TArray<int32> TwistBones;
		for (const int32 AddedBoneIndex : SkelBoneIndices)
		{
			for (int32 Index = 0; Index < RefSkel.GetNum(); ++Index)
			{
				if (RefSkel.GetParentIndex(Index) == AddedBoneIndex &&
					RefSkel.GetBoneName(Index).ToString().Contains(TwistSubString))
				{
					TwistBones.AddUnique(Index);
				}
			}
		}
		for (const int32 TwistBoneIndex : TwistBones)
		{
			SkelBoneIndices.AddUnique(TwistBoneIndex);
		}
	}

	void FNeuralMorphEditorModel::GenerateBoneMaskInfo(int32 InputInfoBoneIndex, int32 HierarchyDepth)
	{
		UNeuralMorphModel* NeuralMorphModel = GetNeuralMorphModel();
		USkeletalMesh* SkeletalMesh = Model->GetSkeletalMesh();	
		UNeuralMorphInputInfo* NeuralMorphInputInfo = Cast<UNeuralMorphInputInfo>(GetEditorInputInfo());
		const FReferenceSkeleton& RefSkel = SkeletalMesh->GetRefSkeleton();
		const TArray<FName>& BoneNames = NeuralMorphInputInfo->GetBoneNames();

		// Make sure we have a valid bone name.
		const FName BoneName = BoneNames[InputInfoBoneIndex];
		if (BoneName == NAME_None)
		{
			return;
		}

		// Get the bone index inside our skeleton.
		const int32 SkeletonBoneIndex = RefSkel.FindBoneIndex(BoneName);
		if (SkeletonBoneIndex == INDEX_NONE)
		{
			UE_LOGF(LogNeuralMorphModel, Warning, "Bone '%ls' cannot be found in the SkeletalMesh '%ls', ignoring during mask generation.", *BoneName.ToString(), *SkeletalMesh->GetName());
			return;
		}

		// Build the list of bones that are required for the mask.
		TArray<int32> BonesAdded;
		RecursiveAddBoneToMaskUpwards(RefSkel, SkeletonBoneIndex, HierarchyDepth, BonesAdded);
		RecursiveAddBoneToMaskDownwards(RefSkel, SkeletonBoneIndex, HierarchyDepth, BonesAdded);
		AddTwistBones(RefSkel, BonesAdded);

		// Now that we know which bones we need, add them to the mask.
		FMLDeformerMaskInfo* MaskInfo = NeuralMorphModel->BoneMaskInfoMap.Find(BoneName);
		if (MaskInfo == nullptr)
		{
			MaskInfo = &NeuralMorphModel->BoneMaskInfoMap.Add(BoneName, FMLDeformerMaskInfo());
		}
		MaskInfo->BoneNames.Reset();
		for (int32 BoneIndex : BonesAdded)
		{
			const FName MaskBoneName = RefSkel.GetBoneName(BoneIndex);
			MaskInfo->BoneNames.Add(MaskBoneName);
		}
	}

	void FNeuralMorphEditorModel::GenerateBoneGroupMaskInfo(int32 InputInfoBoneGroupIndex, int32 HierarchyDepth)
	{
		UNeuralMorphModel* NeuralMorphModel = GetNeuralMorphModel();
		USkeletalMesh* SkeletalMesh = Model->GetSkeletalMesh();	
		const FReferenceSkeleton& RefSkel = SkeletalMesh->GetRefSkeleton();

		// For all bones.
		TArray<int32> BonesAdded;
		const FNeuralMorphBoneGroup& BoneGroup = NeuralMorphModel->GetBoneGroups()[InputInfoBoneGroupIndex];
		for (int32 BoneIndex = 0; BoneIndex < BoneGroup.BoneNames.Num(); ++BoneIndex)
		{
			// Get the bone name.
			const FName BoneName = BoneGroup.BoneNames[BoneIndex].BoneName;
			if (BoneName == NAME_None)
			{
				UE_LOGF(LogNeuralMorphModel, Warning, "Bone index %d inside bone group %d has its name set to None, ignoring the bone inside the group's mask.", BoneIndex, InputInfoBoneGroupIndex);
				continue;
			}

			// Get the bone index inside our skeleton.
			const int32 SkeletonBoneIndex = RefSkel.FindBoneIndex(BoneName);
			if (SkeletonBoneIndex == INDEX_NONE)
			{
				UE_LOGF(LogNeuralMorphModel, Warning, "Bone '%ls' cannot be found in the SkeletalMesh '%ls', ignoring the bone inside the group's mask.", *BoneName.ToString(), *SkeletalMesh->GetName());
				continue;
			}

			// Add all bones up and down the chain.
			RecursiveAddBoneToMaskUpwards(RefSkel, SkeletonBoneIndex, HierarchyDepth, BonesAdded);
			RecursiveAddBoneToMaskDownwards(RefSkel, SkeletonBoneIndex, HierarchyDepth, BonesAdded);
			AddTwistBones(RefSkel, BonesAdded);
		}

		// Now that we know which bones we need, add them to the mask.
		FMLDeformerMaskInfo* MaskInfo = NeuralMorphModel->BoneGroupMaskInfoMap.Find(BoneGroup.GroupName);
		if (MaskInfo == nullptr)
		{
			MaskInfo = &NeuralMorphModel->BoneGroupMaskInfoMap.Add(BoneGroup.GroupName, FMLDeformerMaskInfo());
		}
		MaskInfo->BoneNames.Reset();
		for (int32 BoneIndex : BonesAdded)
		{
			const FName MaskBoneName = RefSkel.GetBoneName(BoneIndex);
			MaskInfo->BoneNames.Add(MaskBoneName);
		}
	}

	void FNeuralMorphEditorModel::GenerateBoneMaskInfos(int32 HierarchyDepth)
	{
		check(HierarchyDepth >= 1);
		ResetBoneMaskInfos();

		// Generate a bone mask info for each bone.
		UNeuralMorphInputInfo* NeuralMorphInputInfo = Cast<UNeuralMorphInputInfo>(GetEditorInputInfo());
		const int32 NumBones = NeuralMorphInputInfo->GetNumBones();
		for (int32 Index = 0; Index < NumBones; ++Index)
		{
			GenerateBoneMaskInfo(Index, HierarchyDepth);
		}
	}

	void FNeuralMorphEditorModel::GenerateBoneGroupMaskInfos(int32 HierarchyDepth)
	{
		check(HierarchyDepth >= 1);

		ResetBoneGroupMaskInfos();

		// Generate a mask info for each group.
		UNeuralMorphInputInfo* NeuralMorphInputInfo = Cast<UNeuralMorphInputInfo>(GetEditorInputInfo());
		const int32 NumBoneGroups = NeuralMorphInputInfo->GetBoneGroups().Num();	
		for (int32 Index = 0; Index < NumBoneGroups; ++Index)
		{
			GenerateBoneGroupMaskInfo(Index, HierarchyDepth);
		}
	}

	void FNeuralMorphEditorModel::OnPostInputAssetChanged()
	{
		FMLDeformerMorphModelEditorModel::OnPostInputAssetChanged();
		RemoveNonExistingMaskInfos();
		if (GetNeuralMorphModel()->GetModelMode() == ENeuralMorphMode::Local)
		{
			RebuildEditorMaskInfo();
		}
	}

	void FNeuralMorphEditorModel::RemoveNonExistingMaskInfos()
	{
		UNeuralMorphModel* NeuralMorphModel = GetNeuralMorphModel();

		// Find all mask infos for bones that don't exist anymore.
		TArray<FName> BoneMaskInfosToRemove;
		for (const auto& MaskInfo : NeuralMorphModel->BoneMaskInfoMap)
		{
			const FName BoneName = MaskInfo.Key;
			if (!NeuralMorphModel->GetBoneIncludeList().Contains(BoneName))
			{
				BoneMaskInfosToRemove.Add(BoneName);
			}
		}

		// Remove those mask infos.
		for (const FName BoneName : BoneMaskInfosToRemove)
		{
			NeuralMorphModel->BoneMaskInfoMap.Remove(BoneName);
		}

		// Find all bone groups that don't exist anymore.
		BoneMaskInfosToRemove.Reset();
		for (const auto& MaskInfo : NeuralMorphModel->BoneGroupMaskInfoMap)
		{
			const FName GroupName = MaskInfo.Key;

			bool bFound = false;
			for (const FNeuralMorphBoneGroup& BoneGroup : NeuralMorphModel->GetBoneGroups())
			{
				if (BoneGroup.GroupName == GroupName)
				{
					bFound = true;
					break;
				}
			}

			if (!bFound)
			{
				BoneMaskInfosToRemove.Add(GroupName);
			}
		}

		// Remove those group mask infos.
		for (const FName GroupName : BoneMaskInfosToRemove)
		{
			NeuralMorphModel->BoneGroupMaskInfoMap.Remove(GroupName);
		}
	}

	void FNeuralMorphEditorModel::DebugDrawItemMask(FPrimitiveDrawInterface* PDI, int32 MaskItemIndex, const FVector& DrawOffset)
	{
		FMLDeformerSampler* Sampler = GetSamplerForActiveAnim();
		if (!Sampler)
		{
			return;
		}

		const int32 NumVerts = Model->GetNumBaseMeshVerts();
		const TArray<FVector3f>& UnskinnedPositions = Sampler->GetUnskinnedVertexPositions();
		if (MaskVizItemIndex == INDEX_NONE ||
			NumVerts != Model->GetInputInfo()->GetNumBaseMeshVertices() ||
			UnskinnedPositions.Num() != NumVerts)
		{
			return;
		}

		UNeuralMorphInputInfo* InputInfo = Cast<UNeuralMorphInputInfo>(GetEditorInputInfo());
		const int32 NumMaskItems = InputInfo->GetBoneNames().Num() + InputInfo->GetCurveNames().Num() + InputInfo->GetBoneGroups().Num() + InputInfo->GetCurveGroups().Num();
		const int32 FinalMaskItemIndex = FMath::Clamp<int32>(MaskItemIndex, 0, NumMaskItems - 1);
		const TArrayView<const float> MaskBuffer = InputInfo->GetMaskForItem(FinalMaskItemIndex);

		// Find the mask info.
		FMLDeformerMaskInfo* MaskInfo = nullptr;
		if (FinalMaskItemIndex < InputInfo->GetBoneNames().Num())	// If we have a bone.
		{
			if (InputInfo->GetBoneNames().IsValidIndex(FinalMaskItemIndex))
			{
				MaskInfo = GetNeuralMorphModel()->BoneMaskInfoMap.Find(InputInfo->GetBoneNames()[FinalMaskItemIndex]);
			}
		}
		else // It's a bone group.
		{
			const int32 GroupIndex = FinalMaskItemIndex - (InputInfo->GetBoneNames().Num() + InputInfo->GetCurveNames().Num());
			if (InputInfo->GetBoneGroups().IsValidIndex(GroupIndex))
			{
				const FName GroupName = InputInfo->GetBoneGroups()[GroupIndex].GroupName;
				MaskInfo = GetNeuralMorphModel()->BoneGroupMaskInfoMap.Find(GroupName);
			}
		}

		if (!MaskBuffer.IsEmpty())
		{
			check(MaskBuffer.Num() == NumVerts);
			check(NumVerts == UnskinnedPositions.Num());

			const FLinearColor OrgIncludedColor = (MaskInfo && MaskInfo->MaskMode == EMLDeformerMaskingMode::VertexAttribute) ? FMLDeformerEditorStyle::Get().GetColor("MLDeformer.Morphs.MaskIncludedVertexColorPainted") : FMLDeformerEditorStyle::Get().GetColor("MLDeformer.Morphs.MaskIncludedVertexColor");
			const FLinearColor ExcludedColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.Morphs.MaskExcludedVertexColor");
			for (int32 VertexIndex = 0; VertexIndex < NumVerts; ++VertexIndex)
			{
				const FVector StartPoint = FVector(UnskinnedPositions[VertexIndex]) + DrawOffset;
				if (MaskBuffer[VertexIndex] > 0.0f)
				{
					const FLinearColor IncludedColor = OrgIncludedColor * FMath::Clamp<float>(MaskBuffer[VertexIndex], 0.0f, 1.0f);
					PDI->DrawPoint(StartPoint, IncludedColor, 1.0f, 0);
				}
				else
				{
					PDI->DrawPoint(StartPoint, ExcludedColor, 0.75f, 0);
				}
			}
		}
	}

	void FNeuralMorphEditorModel::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
	{
		FMLDeformerMorphModelEditorModel::Render(View, Viewport, PDI);

		const UNeuralMorphModelVizSettings* VizSettings = Cast<UNeuralMorphModelVizSettings>(Model->GetVizSettings());
		check(VizSettings);
		if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TestData &&
			VizSettings->MaskVizMode != ENeuralMorphMaskVizMode::Off &&
			GetNeuralMorphModel()->GetModelMode() == ENeuralMorphMode::Local &&
			MaskVizItemIndex != INDEX_NONE)
		{
			bool bDrawMask = true;
			if (VizSettings->MaskVizMode == ENeuralMorphMaskVizMode::WhenInFocus && !InputWidget->HasFocusedDescendants())
			{
				bDrawMask = false;
			}

			if (bDrawMask)
			{
				FVector DrawOffset = -VizSettings->GetMeshSpacingOffsetVector();
				if (VizSettings->GetDrawMorphTargets() && !GetMorphModel()->GetMorphTargetDeltas().IsEmpty())
				{
					DrawOffset *= 2.0f;
				}

				DebugDrawItemMask(PDI, MaskVizItemIndex, DrawOffset);
			}
		}
	}
}	// namespace UE::NeuralMorphModel

#undef LOCTEXT_NAMESPACE
