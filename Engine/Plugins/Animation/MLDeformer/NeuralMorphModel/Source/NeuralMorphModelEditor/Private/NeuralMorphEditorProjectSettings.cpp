// Copyright Epic Games, Inc. All Rights Reserved.

#include "NeuralMorphEditorProjectSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NeuralMorphEditorProjectSettings)

#define LOCTEXT_NAMESPACE "NeuralMorphEditorProjectSettings"

const UNeuralMorphEditorProjectSettings* UNeuralMorphEditorProjectSettings::Get()
{ 
	return GetDefault<UNeuralMorphEditorProjectSettings>();
}

#if WITH_EDITOR
	FText UNeuralMorphEditorProjectSettings::GetSectionText() const 
	{ 
		return LOCTEXT("NeuralMorphModelProjectSettingsName", "Neural Morph Model"); 
	}

	FText UNeuralMorphEditorProjectSettings::GetSectionDescription() const
	{
		return LOCTEXT("NeuralMorphModelProjectSettingsDescription", "Configure project wide settings related to the Neural Morph Model, which is part of the ML Deformer.");
	}
#endif

#undef LOCTEXT_NAMESPACE
