// Copyright Epic Games, Inc. All Rights Reserved.

#include "NeuralMorphModelDetails.h"
#include "NeuralMorphModel.h"
#include "NeuralMorphEditorModel.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"

#define LOCTEXT_NAMESPACE "NeuralMorphModelDetails"

namespace UE::NeuralMorphModel
{
	using namespace UE::MLDeformer;

	TSharedRef<IDetailCustomization> FNeuralMorphModelDetails::MakeInstance()
	{
		return MakeShareable(new FNeuralMorphModelDetails());
	}

	void FNeuralMorphModelDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
	{
		// Create all the detail categories and add the properties of the base class.
		FMLDeformerMorphModelDetails::CustomizeDetails(DetailBuilder);

		if (!EditorModel)
		{
			return;
		}

		UNeuralMorphModel* NeuralMorphModel = Cast<UNeuralMorphModel>(Model);
		check(NeuralMorphModel);

		TrainingSettingsCategoryBuilder->AddProperty(GET_MEMBER_NAME_CHECKED(UNeuralMorphModel, Mode), UNeuralMorphModel::StaticClass())
			.Visibility(NeuralMorphModel->SupportsGlobalModeOnly() ? EVisibility::Collapsed : EVisibility::Visible);

		// Local mode settings.
		TrainingSettingsCategoryBuilder->AddProperty(GET_MEMBER_NAME_CHECKED(UNeuralMorphModel, LocalNumMorphTargetsPerBone), UNeuralMorphModel::StaticClass())
			.Visibility(NeuralMorphModel->GetModelMode() == ENeuralMorphMode::Local ? EVisibility::Visible : EVisibility::Collapsed);
		TrainingSettingsCategoryBuilder->AddProperty(GET_MEMBER_NAME_CHECKED(UNeuralMorphModel, LocalNumHiddenLayers), UNeuralMorphModel::StaticClass())
			.Visibility(NeuralMorphModel->GetModelMode() == ENeuralMorphMode::Local ? EVisibility::Visible : EVisibility::Collapsed);
		TrainingSettingsCategoryBuilder->AddProperty(GET_MEMBER_NAME_CHECKED(UNeuralMorphModel, LocalNumNeuronsPerLayer), UNeuralMorphModel::StaticClass())
			.Visibility(NeuralMorphModel->GetModelMode() == ENeuralMorphMode::Local ? EVisibility::Visible : EVisibility::Collapsed);

		// Global mode settings.
		TrainingSettingsCategoryBuilder->AddProperty(GET_MEMBER_NAME_CHECKED(UNeuralMorphModel, GlobalNumMorphTargets), UNeuralMorphModel::StaticClass())
			.Visibility(NeuralMorphModel->GetModelMode() == ENeuralMorphMode::Global ? EVisibility::Visible : EVisibility::Collapsed);
		TrainingSettingsCategoryBuilder->AddProperty(GET_MEMBER_NAME_CHECKED(UNeuralMorphModel, GlobalNumHiddenLayers), UNeuralMorphModel::StaticClass())
			.Visibility(NeuralMorphModel->GetModelMode()== ENeuralMorphMode::Global ? EVisibility::Visible : EVisibility::Collapsed);
		TrainingSettingsCategoryBuilder->AddProperty(GET_MEMBER_NAME_CHECKED(UNeuralMorphModel, GlobalNumNeuronsPerLayer), UNeuralMorphModel::StaticClass())
			.Visibility(NeuralMorphModel->GetModelMode() == ENeuralMorphMode::Global ? EVisibility::Visible : EVisibility::Collapsed);

		TrainingSettingsCategoryBuilder->AddProperty(GET_MEMBER_NAME_CHECKED(UNeuralMorphModel, NumIterations), UNeuralMorphModel::StaticClass());
		TrainingSettingsCategoryBuilder->AddProperty(GET_MEMBER_NAME_CHECKED(UNeuralMorphModel, BatchSize), UNeuralMorphModel::StaticClass());
		TrainingSettingsCategoryBuilder->AddProperty(GET_MEMBER_NAME_CHECKED(UNeuralMorphModel, LearningRate), UNeuralMorphModel::StaticClass());
		TrainingSettingsCategoryBuilder->AddProperty(GET_MEMBER_NAME_CHECKED(UNeuralMorphModel, RegularizationFactor), UNeuralMorphModel::StaticClass());
		TrainingSettingsCategoryBuilder->AddProperty(GET_MEMBER_NAME_CHECKED(UNeuralMorphModel, SmoothLossBeta), UNeuralMorphModel::StaticClass());
		TrainingSettingsCategoryBuilder->AddProperty(GET_MEMBER_NAME_CHECKED(UNeuralMorphModel, bEnableBoneMasks), UNeuralMorphModel::StaticClass());
		TrainingSettingsCategoryBuilder->AddProperty(UNeuralMorphModel::GetSkinningModePropertyName(), UNeuralMorphModel::StaticClass());
	}
}	// namespace UE::NeuralMorphModel

#undef LOCTEXT_NAMESPACE
