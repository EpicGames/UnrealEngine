// Copyright Epic Games, Inc. All Rights Reserved.

#include "NeuralMorphModelVizSettingsDetails.h"
#include "NeuralMorphModelDetails.h"
#include "NeuralMorphEditorModel.h"
#include "NeuralMorphModel.h"
#include "SNeuralMorphInputWidget.h"
#include "SNeuralMorphBoneGroupsWidget.h"
#include "SNeuralMorphCurveGroupsWidget.h"
#include "MLDeformerEditorModule.h"
#include "Modules/ModuleManager.h"
#include "EditorModeRegistry.h"

#define LOCTEXT_NAMESPACE "NeuralMorphModelEditorModule"

namespace UE::NeuralMorphModel
{
	class FNeuralMorphModelEditorModule
		: public IModuleInterface
	{
	public:
		// IModuleInterface overrides.
		void StartupModule() override;
		void ShutdownModule() override;
		// ~END IModuleInterface overrides.
	};
}
IMPLEMENT_MODULE(UE::NeuralMorphModel::FNeuralMorphModelEditorModule, NeuralMorphModelEditor)

namespace UE::NeuralMorphModel
{
	using namespace UE::MLDeformer;


	void FNeuralMorphModelEditorModule::StartupModule()
	{
		// Register object detail customizations.
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout("NeuralMorphModelVizSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FNeuralMorphModelVizSettingsDetails::MakeInstance));
		PropertyModule.RegisterCustomClassLayout("NeuralMorphModel", FOnGetDetailCustomizationInstance::CreateStatic(&FNeuralMorphModelDetails::MakeInstance));
		PropertyModule.NotifyCustomizationModuleChanged();

		// Register our custom ML deformer model to the model registry in the ML Deformer Framework.
		FMLDeformerEditorModule& EditorModule = FModuleManager::LoadModuleChecked<FMLDeformerEditorModule>("MLDeformerFrameworkEditor");
		FMLDeformerEditorModelRegistry& ModelRegistry = EditorModule.GetModelRegistry();
		ModelRegistry.RegisterEditorModel(UNeuralMorphModel::StaticClass(), FOnGetEditorModelInstance::CreateStatic(&FNeuralMorphEditorModel::MakeInstance), /*ModelPriority*/100);

		FNeuralMorphInputWidgetCommands::Register();
		FNeuralMorphBoneGroupsCommands::Register();
		FNeuralMorphCurveGroupsCommands::Register();
	}

	void FNeuralMorphModelEditorModule::ShutdownModule()
	{
		// Unregister our ML Deformer model.
		if (FModuleManager::Get().IsModuleLoaded(TEXT("MLDeformerFrameworkEditor")))
		{
			FMLDeformerEditorModule& EditorModule = FModuleManager::GetModuleChecked<FMLDeformerEditorModule>("MLDeformerFrameworkEditor");
			FMLDeformerEditorModelRegistry& ModelRegistry = EditorModule.GetModelRegistry();
			ModelRegistry.UnregisterEditorModel(UNeuralMorphModel::StaticClass());
		}

		// Unregister object detail customizations for this model.
		if (FModuleManager::Get().IsModuleLoaded(TEXT("PropertyEditor")))
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
			PropertyModule.UnregisterCustomClassLayout(TEXT("NeuralMorphModelVizSettings"));
			PropertyModule.UnregisterCustomClassLayout(TEXT("NeuralMorphModel"));
			PropertyModule.NotifyCustomizationModuleChanged();
		}
	}
}	// namespace UE::NeuralMorphModel

#undef LOCTEXT_NAMESPACE
