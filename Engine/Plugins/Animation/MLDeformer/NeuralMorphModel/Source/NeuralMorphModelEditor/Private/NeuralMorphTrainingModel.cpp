// Copyright Epic Games, Inc. All Rights Reserved.

#include "NeuralMorphTrainingModel.h"
#include "NeuralMorphEditorModel.h"
#include "NeuralMorphModel.h"
#include "NeuralMorphInputInfo.h"
#include "MLDeformerSampler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NeuralMorphTrainingModel)

void UNeuralMorphTrainingModel::Init(UE::MLDeformer::FMLDeformerEditorModel* InEditorModel)
{
	Super::Init(InEditorModel);

	UNeuralMorphModel* NeuralMorphModel = Cast<UNeuralMorphModel>(EditorModel->GetModel());
	check(NeuralMorphModel);
	for (int32 AnimIndex = 0; AnimIndex < EditorModel->GetNumTrainingInputAnims(); ++AnimIndex)
	{
		UE::MLDeformer::FMLDeformerSampler* Sampler = EditorModel->GetSamplerForTrainingAnim(AnimIndex);
		if (Sampler)
		{
			Sampler->SetSkinningMode(NeuralMorphModel->GetSkinningMode());
		}
	}
}

int32 UNeuralMorphTrainingModel::GetNumBoneGroups() const
{
	UNeuralMorphInputInfo* InputInfo = Cast<UNeuralMorphInputInfo>(EditorModel->GetEditorInputInfo());
	return InputInfo ? InputInfo->GetBoneGroups().Num() : 0;
}

int32 UNeuralMorphTrainingModel::GetNumCurveGroups() const
{
	UNeuralMorphInputInfo* InputInfo = Cast<UNeuralMorphInputInfo>(EditorModel->GetEditorInputInfo());	
	return InputInfo ? InputInfo->GetCurveGroups().Num() : 0;
}

TArray<int32> UNeuralMorphTrainingModel::GenerateBoneGroupIndices() const
{
	UNeuralMorphInputInfo* InputInfo = Cast<UNeuralMorphInputInfo>(EditorModel->GetEditorInputInfo());
	if (InputInfo == nullptr)
	{
		return TArray<int32>();
	}

	TArray<int32> OutBoneGroupIndices;
	InputInfo->GenerateBoneGroupIndices(OutBoneGroupIndices);
	return MoveTemp(OutBoneGroupIndices);
}

TArray<int32> UNeuralMorphTrainingModel::GenerateCurveGroupIndices() const
{
	UNeuralMorphInputInfo* InputInfo = Cast<UNeuralMorphInputInfo>(EditorModel->GetEditorInputInfo());
	if (InputInfo == nullptr)
	{
		return TArray<int32>();
	}

	TArray<int32> OutCurveGroupIndices;
	InputInfo->GenerateCurveGroupIndices(OutCurveGroupIndices);
	return MoveTemp(OutCurveGroupIndices);
}

TArray<float> UNeuralMorphTrainingModel::GetMorphTargetMasks() const
{
	using namespace UE::NeuralMorphModel;
	FNeuralMorphEditorModel* NeuralMorphEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);
	UNeuralMorphModel* NeuralMorphModel = NeuralMorphEditorModel->GetNeuralMorphModel();
	UNeuralMorphInputInfo* NeuralInputInfo = Cast<UNeuralMorphInputInfo>(NeuralMorphEditorModel->GetEditorInputInfo());
	if (NeuralMorphModel->GetModelMode() != ENeuralMorphMode::Local || !NeuralMorphModel->IsBoneMaskingEnabled())
	{
		NeuralInputInfo->GetInputItemMaskBuffer().Empty();
	}
	else	
	{
		NeuralMorphEditorModel->RebuildEditorMaskInfo();
	}

	return NeuralInputInfo->GetInputItemMaskBuffer();
}
