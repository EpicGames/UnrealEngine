// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNeuralMorphBoneGroupsWidget.h"
#include "NeuralMorphModel.h"
#include "NeuralMorphEditorModel.h"
#include "NeuralMorphInputInfo.h"
#include "SNeuralMorphInputWidget.h"
#include "SMLDeformerBonePickerDialog.h"
#include "MLDeformerEditorStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Engine/SkeletalMesh.h"
#include "ScopedTransaction.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"
#include "HAL/PlatformApplicationMisc.h"

#define LOCTEXT_NAMESPACE "NeuralMorphBoneGroupsWidget"

namespace UE::NeuralMorphModel
{
	FNeuralMorphBoneGroupsCommands::FNeuralMorphBoneGroupsCommands()
		: TCommands<FNeuralMorphBoneGroupsCommands>
	(	"Neural Morph Bone Groups",
		NSLOCTEXT("NeuralMorphBoneGroupsWidget", "NeuralMorphBoneGroupsDesc", "Neural Morph Bone Groups"),
		NAME_None,
		FMLDeformerEditorStyle::Get().GetStyleSetName())
	{
	}

	void FNeuralMorphBoneGroupsCommands::RegisterCommands()
	{
		UI_COMMAND(CreateGroup, "Create New Group", "Create a new bone group.", EUserInterfaceActionType::Button, FInputChord(EKeys::Insert));
		UI_COMMAND(DeleteSelectedItems, "Delete Selected Items", "Deletes the selected bones and/or groups.", EUserInterfaceActionType::Button, FInputChord(EKeys::Delete));
		UI_COMMAND(ClearGroups, "Clear All Groups", "Clears the entire list of bone groups.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(AddBoneToGroup, "Add Bones To Group", "Add new bones to the group.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(CopyBoneGroups, "Copy Input Bone Groups List", "Copies all bone groups, so that you can paste them into another model's input bone groups list.", EUserInterfaceActionType::Button, FInputChord(EKeys::C, EModifierKey::Control));
		UI_COMMAND(PasteBoneGroups, "Paste Input Bones Groups List", "Pastes a previously copied bone group list. This will add bone groups to the existing list, not replace it fully.", EUserInterfaceActionType::Button, FInputChord(EKeys::V, EModifierKey::Control));
	}

	void SNeuralMorphBoneGroupsWidget::BindCommands(TSharedPtr<FUICommandList> CommandList)
	{
		const FNeuralMorphBoneGroupsCommands& GroupCommands = FNeuralMorphBoneGroupsCommands::Get();
		CommandList->MapAction(GroupCommands.CreateGroup, FExecuteAction::CreateSP(this, &SNeuralMorphBoneGroupsWidget::OnCreateBoneGroup));
		CommandList->MapAction(GroupCommands.DeleteSelectedItems, FExecuteAction::CreateSP(this, &SNeuralMorphBoneGroupsWidget::OnDeleteSelectedItems));
		CommandList->MapAction(GroupCommands.ClearGroups, FExecuteAction::CreateSP(this, &SNeuralMorphBoneGroupsWidget::OnClearBoneGroups));
		CommandList->MapAction(GroupCommands.AddBoneToGroup, FExecuteAction::CreateSP(this, &SNeuralMorphBoneGroupsWidget::OnAddBoneToGroup));
		CommandList->MapAction(GroupCommands.CopyBoneGroups, FExecuteAction::CreateSP(this, &SNeuralMorphBoneGroupsWidget::OnCopyBoneGroups));
		CommandList->MapAction(GroupCommands.PasteBoneGroups, FExecuteAction::CreateSP(this, &SNeuralMorphBoneGroupsWidget::OnPasteBoneGroups));
	}

	void SNeuralMorphBoneGroupsWidget::Construct(const FArguments& InArgs)
	{
		EditorModel = InArgs._EditorModel;
		InputWidget = InArgs._InputWidget;

		STreeView<TSharedPtr<FNeuralMorphBoneGroupsTreeElement>>::FArguments SuperArgs;
		SuperArgs.TreeItemsSource(&RootElements);
		SuperArgs.SelectionMode(ESelectionMode::Multi);
		SuperArgs.OnGenerateRow(this, &SNeuralMorphBoneGroupsWidget::MakeTableRowWidget);
		SuperArgs.OnGetChildren(this, &SNeuralMorphBoneGroupsWidget::HandleGetChildrenForTree);
		SuperArgs.OnSelectionChanged(this, &SNeuralMorphBoneGroupsWidget::OnSelectionChanged);
		SuperArgs.OnContextMenuOpening(this, &SNeuralMorphBoneGroupsWidget::CreateContextMenuWidget);
		SuperArgs.HighlightParentNodesForSelection(false);
		SuperArgs.AllowInvisibleItemSelection(true);  // Without this we deselect everything when we filter or we collapse.

		STreeView<TSharedPtr<FNeuralMorphBoneGroupsTreeElement>>::Construct(SuperArgs);

		RefreshTree(false);
	}

	void SNeuralMorphBoneGroupsWidget::OnSelectionChanged(TSharedPtr<FNeuralMorphBoneGroupsTreeElement> Selection, ESelectInfo::Type SelectInfo)
	{
		const TSharedPtr<SNeuralMorphInputWidget> NeuralInputWidget = StaticCastSharedPtr<SNeuralMorphInputWidget>(InputWidget);
		if (NeuralInputWidget.IsValid())
		{
			NeuralInputWidget->OnSelectInputBoneGroup(Selection);
		}
	}

	TSharedPtr<SWidget> SNeuralMorphBoneGroupsWidget::CreateContextMenuWidget() const
	{
		const FNeuralMorphBoneGroupsCommands& Actions = FNeuralMorphBoneGroupsCommands::Get();

		TSharedPtr<FUICommandList> CommandList = StaticCastSharedPtr<SNeuralMorphInputWidget>(InputWidget)->GetBoneGroupsCommandList();
		FMenuBuilder Menu(true, CommandList);

		const TArray<TSharedPtr<FNeuralMorphBoneGroupsTreeElement>> CurSelectedItems = GetSelectedItems();
		Menu.BeginSection("BoneGroupActions", LOCTEXT("BoneGroupActionsHeading", "Bone Group Actions"));
		{
			Menu.AddMenuEntry(Actions.CreateGroup);
			
			if (CurSelectedItems.Num() == 1 && CurSelectedItems[0]->IsGroup())
			{
				Menu.AddMenuEntry(Actions.AddBoneToGroup);
			}

			if (!CurSelectedItems.IsEmpty())
			{			
				Menu.AddMenuEntry(Actions.DeleteSelectedItems);
			}

			if (!GetRootElements().IsEmpty())
			{
				Menu.AddMenuEntry(Actions.ClearGroups);
			}
		}
		Menu.EndSection();

		// Add the bone mask settings.
		TSharedPtr<SNeuralMorphInputWidget> NeuralInputWidget = StaticCastSharedPtr<SNeuralMorphInputWidget>(InputWidget);
		if (NeuralInputWidget.IsValid())
		{
			NeuralInputWidget->AddInputBoneGroupsMenuItems(Menu);
		}

		Menu.BeginSection("CopyPasteActions", LOCTEXT("CopyPasteActionsHeading", "Copy/Paste Actions"));
		{
			if (!RootElements.IsEmpty())
			{
				Menu.AddMenuEntry(Actions.CopyBoneGroups);
			}

			TSharedPtr<FJsonObject> JsonObject;
			if (InputWidget->HasValidClipBoardData(TEXT("BoneGroups"), JsonObject))
			{
				Menu.AddMenuEntry(Actions.PasteBoneGroups);
			}
		}
		Menu.EndSection();
		
		return Menu.MakeWidget();
	}

	void SNeuralMorphBoneGroupsWidget::OnCopyBoneGroups() const
	{
		const UNeuralMorphModel* NeuralMorphModel = EditorModel->GetNeuralMorphModel();
		if (NeuralMorphModel->GetBoneGroups().IsEmpty())
		{
			return;
		}

		const TSharedRef<FJsonObject> JsonObject = MakeShared<FJsonObject>();
		JsonObject->SetStringField(TEXT("DataId"), TEXT("BoneGroups"));

		// Create each bone group.
		TArray<TSharedPtr<FJsonValue>> BoneGroupsArray;
		for (const FNeuralMorphBoneGroup& BoneGroup : NeuralMorphModel->GetBoneGroups())
		{
			TSharedRef<FJsonObject> JsonBoneGroup = MakeShared<FJsonObject>();
			JsonBoneGroup->SetStringField(TEXT("GroupName"), BoneGroup.GroupName.ToString());
    
			TArray<TSharedPtr<FJsonValue>> BoneNames;
			for (const FBoneReference& BoneName : BoneGroup.BoneNames)
			{
				if (BoneName.BoneName.IsValid() && !BoneName.BoneName.IsNone())
				{
					BoneNames.Add(MakeShared<FJsonValueString>(BoneName.BoneName.ToString()));
				}
			}
	
			JsonBoneGroup->SetArrayField(TEXT("BoneNames"), BoneNames);
			BoneGroupsArray.Add(MakeShared<FJsonValueObject>(JsonBoneGroup));
		}

		JsonObject->SetArrayField(TEXT("BoneGroups"), BoneGroupsArray);

		// Convert the JSON to a string and copy it to the clipboard.
		FString OutputString;
		const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);
		if (!FJsonSerializer::Serialize(JsonObject, Writer))
		{
			UE_LOGF(LogMLDeformer, Error, "Failed to serialize JSON on Copy action in the Input Bone Groups widget.");
			return;
		}
		FPlatformApplicationMisc::ClipboardCopy(*OutputString);
	}

	void SNeuralMorphBoneGroupsWidget::OnPasteBoneGroups()
	{
		FString Text;
		FPlatformApplicationMisc::ClipboardPaste(Text);

		TSharedPtr<FJsonObject> JsonObject;
		if (GetInputWidget()->HasValidClipBoardData(TEXT("BoneGroups"), JsonObject) && JsonObject.IsValid())
		{
			const TArray<TSharedPtr<FJsonValue>>* BoneGroupsArray;	
			if (JsonObject->TryGetArrayField(TEXT("BoneGroups"), BoneGroupsArray))
			{
				UNeuralMorphModel* NeuralMorphModel = EditorModel->GetNeuralMorphModel();
				FScopedTransaction Transaction(TEXT("SNeuralMorphBoneGroupsWidget"), LOCTEXT("PasteBoneGroupText", "Paste Bone Groups"), NeuralMorphModel);
				NeuralMorphModel->Modify();
	
				for (const TSharedPtr<FJsonValue>& GroupValue : *BoneGroupsArray)
				{
					TSharedPtr<FJsonObject> GroupObject = GroupValue->AsObject();
					if (!GroupObject.IsValid())
					{
						UE_LOGF(LogNeuralMorphModel, Error, "Skipping invalid group object when pasting bone groups.");
						continue;
					}

					FString GroupNameString;
					if (!GroupObject->TryGetStringField(TEXT("GroupName"), GroupNameString))
					{
						UE_LOGF(LogNeuralMorphModel, Error, "Failed to read bone group name when pasting bone groups.");
						continue;
					}
					
					const TArray<TSharedPtr<FJsonValue>>* BoneArray;
					if (!GroupObject->TryGetArrayField(TEXT("BoneNames"), BoneArray))
					{
						UE_LOGF(LogNeuralMorphModel, Error, "Failed to read bone group bone names array when pasting bone groups.");
						continue;
					}

					// Check if the copied group name is unique or if we have to generate a new one.
					bool bIsUniqueName = true;
					for (const FNeuralMorphBoneGroup& Group : NeuralMorphModel->BoneGroups)
					{
						if (Group.GroupName == GroupNameString)
						{
							bIsUniqueName = false;
							break;
						}
					}
					
					const FName GroupName = bIsUniqueName ? FName(GroupNameString) : EditorModel->GenerateUniqueBoneGroupName();
					FNeuralMorphBoneGroup& BoneGroup = NeuralMorphModel->BoneGroups.AddDefaulted_GetRef();
					BoneGroup.GroupName = GroupName;

					// Add the bones to the newly created group.
					for (const TSharedPtr<FJsonValue>& BoneValue : *BoneArray)
					{
						const FString BoneName = BoneValue->AsString();
						BoneGroup.BoneNames.AddUnique(FName(BoneName));
					}
				}
			}
			
			RefreshTree(true);
			EditorModel->RebuildEditorMaskInfo();
		}
	}
	

	void SNeuralMorphBoneGroupsWidget::OnCreateBoneGroup()
	{
		using namespace UE::MLDeformer;

		USkeletalMesh* SkelMesh = EditorModel->GetModel()->GetSkeletalMesh();
		if (!SkelMesh)
		{
			return;
		}

		// Find the group we want to add something to.
		UNeuralMorphModel* NeuralMorphModel = EditorModel->GetNeuralMorphModel();

		const UNeuralMorphInputInfo* InputInfo = Cast<const UNeuralMorphInputInfo>(EditorModel->GetEditorInputInfo());
		check(InputInfo);

		TSharedPtr<SMLDeformerBonePickerDialog> Dialog = 
			SNew(SMLDeformerBonePickerDialog)
			.RefSkeleton(&SkelMesh->GetRefSkeleton())
			.AllowMultiSelect(true)
			.IncludeList(InputInfo->GetBoneNames())
			.ExtraWidget(InputWidget->GetExtraBonePickerWidget());

		Dialog->ShowModal();

		const TArray<FName>& BoneNames = Dialog->GetPickedBoneNames();
		if (BoneNames.IsEmpty())
		{
			return;
		}

		FScopedTransaction Transaction(TEXT("SNeuralMorphBoneGroupsWidget"), LOCTEXT("CreateBoneGroupText", "Create Bone Group"), NeuralMorphModel);
		NeuralMorphModel->Modify();

		NeuralMorphModel->BoneGroups.AddDefaulted();
		FNeuralMorphBoneGroup& BoneGroup = NeuralMorphModel->BoneGroups.Last();
		BoneGroup.GroupName = EditorModel->GenerateUniqueBoneGroupName();
		for (const FName BoneName : BoneNames)
		{
			BoneGroup.BoneNames.AddUnique(BoneName);
		}

		EditorModel->UpdateEditorInputInfo();
		RefreshTree(true);
		EditorModel->RebuildEditorMaskInfo();
	}

	void SNeuralMorphBoneGroupsWidget::OnDeleteSelectedItems()
	{
		const TArray<TSharedPtr<FNeuralMorphBoneGroupsTreeElement>> CurSelectedItems = GetSelectedItems();

		TArray<int32> GroupsToRemove;

		TArray<TSharedPtr<FNeuralMorphBoneGroupsTreeElement>> SelectedGroups;
		TArray<TSharedPtr<FNeuralMorphBoneGroupsTreeElement>> SelectedBones;

		// Check if the selection contains bones and/or groups.
		for (const TSharedPtr<FNeuralMorphBoneGroupsTreeElement>& SelectedItem : CurSelectedItems)
		{
			if (SelectedItem->IsGroup())
			{
				SelectedGroups.Add(SelectedItem);
				GroupsToRemove.Add(SelectedItem->GroupIndex);
			}
			else
			{
				SelectedBones.Add(SelectedItem);
			}
		}

		UNeuralMorphModel* NeuralMorphModel = EditorModel->GetNeuralMorphModel();

		FScopedTransaction Transaction(TEXT("SNeuralMorphBoneGroupsWidget"), LOCTEXT("RemoveBoneGroupItemsText", "Remove Bone Group Items"), NeuralMorphModel);
		NeuralMorphModel->Modify();

		// Remove all selected bones.
		if (!SelectedBones.IsEmpty())
		{
			for (const TSharedPtr<FNeuralMorphBoneGroupsTreeElement>& BoneItem : SelectedBones)
			{
				if (BoneItem.IsValid() && BoneItem->ParentGroup.IsValid())
				{
					const TSharedPtr<FNeuralMorphBoneGroupsTreeElement>& ParentGroup = BoneItem->ParentGroup.Pin();

					if (BoneItem->Name.IsNone())
					{
						// Remove all the none items.
						NeuralMorphModel->BoneGroups[ParentGroup->GroupIndex].BoneNames.RemoveAll(
							[](const FBoneReference& Item) 
							{
								return Item.BoneName.IsNone();
							});
					}
					else
					{
						NeuralMorphModel->BoneGroups[ParentGroup->GroupIndex].BoneNames.Remove(BoneItem->Name);
						FMLDeformerMaskInfo* MaskInfo = NeuralMorphModel->BoneGroupMaskInfoMap.Find(ParentGroup->Name);
						if (MaskInfo)
						{
							MaskInfo->BoneNames.Remove(BoneItem->Name);
						}
					}
				}
			}
		}

		// Sort group indices big to small (back to front).
		if (!GroupsToRemove.IsEmpty())
		{
			GroupsToRemove.Sort([](const int32& A, const int32& B){ return A > B; });

			// Remove the items, back to front.
			for (const int32 Index : GroupsToRemove)
			{
				NeuralMorphModel->BoneGroupMaskInfoMap.Remove(NeuralMorphModel->BoneGroups[Index].GroupName);
				NeuralMorphModel->BoneGroups.RemoveAt(Index);
			}
		}

		EditorModel->UpdateEditorInputInfo();
		RefreshTree(true);
		EditorModel->RebuildEditorMaskInfo();
	}

	void SNeuralMorphBoneGroupsWidget::OnClearBoneGroups()
	{
		UNeuralMorphModel* NeuralMorphModel = EditorModel->GetNeuralMorphModel();
		FScopedTransaction Transaction(TEXT("SNeuralMorphBoneGroupsWidget"), LOCTEXT("ClearBoneGroupItemsText", "Clear Bone Groups"), NeuralMorphModel);
		NeuralMorphModel->Modify();

		NeuralMorphModel->BoneGroups.Empty();
		NeuralMorphModel->BoneGroupMaskInfoMap.Empty();
		EditorModel->UpdateEditorInputInfo();
		RefreshTree(true);
		EditorModel->RebuildEditorMaskInfo();
	}

	void SNeuralMorphBoneGroupsWidget::OnAddBoneToGroup()
	{
		using namespace UE::MLDeformer;

		USkeletalMesh* SkelMesh = EditorModel->GetModel()->GetSkeletalMesh();
		if (!SkelMesh)
		{
			return;
		}

		// Find the group we want to add something to.
		check(GetSelectedItems().Num() == 1);
		const int32 GroupIndex = GetSelectedItems()[0]->GroupIndex;
		check(GroupIndex != INDEX_NONE);
		UNeuralMorphModel* NeuralMorphModel = EditorModel->GetNeuralMorphModel();
		FNeuralMorphBoneGroup& BoneGroup = NeuralMorphModel->BoneGroups[GroupIndex];

		// Build the highlighted bone names list.
		TArray<FName> HighlightedBones;
		HighlightedBones.Reserve(BoneGroup.BoneNames.Num());
		for (const FBoneReference& BoneRef : BoneGroup.BoneNames)
		{
			HighlightedBones.Add(BoneRef.BoneName);
		}

		UNeuralMorphInputInfo* InputInfo = Cast<UNeuralMorphInputInfo>(EditorModel->GetEditorInputInfo());
		check(InputInfo);

		const FLinearColor HighlightColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.InputsWidget.HighlightColor");
		TSharedPtr<SMLDeformerBonePickerDialog> Dialog = 
			SNew(SMLDeformerBonePickerDialog)
			.RefSkeleton(&SkelMesh->GetRefSkeleton())
			.AllowMultiSelect(true)
			.HighlightBoneNames(HighlightedBones)
			.HighlightBoneNamesColor(HighlightColor)
			.IncludeList(InputInfo->GetBoneNames())
			.ExtraWidget(InputWidget->GetExtraBonePickerWidget());

		Dialog->ShowModal();

		const TArray<FName>& BoneNames = Dialog->GetPickedBoneNames();
		if (!BoneNames.IsEmpty())
		{
			FScopedTransaction Transaction(TEXT("SNeuralMorphBoneGroupsWidget"), LOCTEXT("AddBoneGroupItemsText", "Add Bones To Group"), NeuralMorphModel);
			NeuralMorphModel->Modify();

			for (const FName BoneName : BoneNames)
			{
				BoneGroup.BoneNames.AddUnique(BoneName);
			}

			const int32 HierarchyDepth = StaticCastSharedPtr<SNeuralMorphInputWidget>(InputWidget)->GetHierarchyDepth();
			EditorModel->GenerateBoneGroupMaskInfo(GroupIndex, HierarchyDepth);

			EditorModel->UpdateEditorInputInfo();
			RefreshTree(true);
			EditorModel->RebuildEditorMaskInfo();
		}
	}

	int32 SNeuralMorphBoneGroupsWidget::GetNumSelectedGroups() const
	{
		int32 NumSelectedGroups = 0;
		for (const TSharedPtr<FNeuralMorphBoneGroupsTreeElement>& Item : SelectedItems)
		{
			if (Item->IsGroup())
			{
				NumSelectedGroups++;
			}
		}
		return NumSelectedGroups;
	}

	void SNeuralMorphBoneGroupsWidget::AddElement(TSharedPtr<FNeuralMorphBoneGroupsTreeElement> Element, TSharedPtr<FNeuralMorphBoneGroupsTreeElement> ParentElement)
	{
		if (!ParentElement)
		{
			RootElements.Add(Element);
		}
		else
		{
			ParentElement->Children.Add(Element);
			Element->ParentGroup = ParentElement->AsWeak();
		}
	}

	TSharedRef<ITableRow> FNeuralMorphBoneGroupsTreeElement::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FNeuralMorphBoneGroupsTreeElement> InTreeElement, TSharedPtr<SNeuralMorphBoneGroupsWidget> InTreeWidget)
	{
		return SNew(SNeuralMorphBoneGroupsTreeRowWidget, InOwnerTable, InTreeElement, InTreeWidget);
	}

	TSharedRef<ITableRow> SNeuralMorphBoneGroupsWidget::MakeTableRowWidget(TSharedPtr<FNeuralMorphBoneGroupsTreeElement> InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return InItem->MakeTreeRowWidget(OwnerTable, InItem.ToSharedRef(), SharedThis(this));
	}

	void SNeuralMorphBoneGroupsWidget::HandleGetChildrenForTree(TSharedPtr<FNeuralMorphBoneGroupsTreeElement> InItem, TArray<TSharedPtr<FNeuralMorphBoneGroupsTreeElement>>& OutChildren)
	{
		OutChildren = InItem.Get()->Children;
	}

	void SNeuralMorphBoneGroupsWidget::UpdateTreeElements()
	{
		RootElements.Reset();

		const FLinearColor ErrorColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.InputsWidget.ErrorColor");

		UNeuralMorphModel* NeuralMorphModel = EditorModel->GetNeuralMorphModel();

		const int32 NumGroups = NeuralMorphModel->GetBoneGroups().Num();
		for (int32 BoneGroupIndex = 0; BoneGroupIndex < NumGroups; ++BoneGroupIndex)
		{
			const FNeuralMorphBoneGroup& BoneGroup = NeuralMorphModel->GetBoneGroups()[BoneGroupIndex];
			bool bGroupHasError = false;

			for (const FBoneReference& BoneRef : BoneGroup.BoneNames)
			{
				if (!BoneRef.BoneName.IsValid() || BoneRef.BoneName.IsNone())
				{
					bGroupHasError = true;
					break;
				}
			}

			// Add the group header.
			TSharedPtr<FNeuralMorphBoneGroupsTreeElement> GroupElement = MakeShared<FNeuralMorphBoneGroupsTreeElement>();
			GroupElement->Name = BoneGroup.GroupName;
			GroupElement->TextColor = bGroupHasError ? FSlateColor(ErrorColor) : FSlateColor::UseForeground();
			GroupElement->GroupIndex = BoneGroupIndex;
			AddElement(GroupElement, nullptr);
			SetItemExpansion(GroupElement, true);

			// Add the items in the group.
			for (int32 BoneIndex = 0; BoneIndex < BoneGroup.BoneNames.Num(); ++BoneIndex)
			{
				const FName BoneName = BoneGroup.BoneNames[BoneIndex].BoneName;
				const bool bBoneHasError = !EditorModel->GetEditorInputInfo()->GetBoneNames().Contains(BoneName);

				TSharedPtr<FNeuralMorphBoneGroupsTreeElement> ItemElement = MakeShared<FNeuralMorphBoneGroupsTreeElement>();
				ItemElement->Name = BoneName;
				ItemElement->TextColor = bBoneHasError ? FSlateColor(ErrorColor) : FSlateColor::UseForeground();
				ItemElement->GroupBoneIndex = BoneIndex;
				AddElement(ItemElement, GroupElement);

				bGroupHasError |= bBoneHasError;
			}

			if (bGroupHasError)
			{
				GroupElement->TextColor = FSlateColor(ErrorColor);
			}
		}
	}

	FReply SNeuralMorphBoneGroupsWidget::OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
	{
		TSharedPtr<FUICommandList> CommandList = StaticCastSharedPtr<SNeuralMorphInputWidget>(InputWidget)->GetBoneGroupsCommandList();
		if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}

		return STreeView<TSharedPtr<FNeuralMorphBoneGroupsTreeElement>>::OnKeyDown(InGeometry, InKeyEvent);
	}

	void SNeuralMorphBoneGroupsWidget::RefreshTree(bool bBroadcastPropertyChanged)
	{
		if (bBroadcastPropertyChanged)
		{
			UNeuralMorphModel* NeuralMorphModel = EditorModel->GetNeuralMorphModel();
			BroadcastModelPropertyChanged(GET_MEMBER_NAME_CHECKED(UNeuralMorphModel, BoneGroups));
		}

		UpdateTreeElements();

		FNeuralMorphEditorModel* NeuralMorphEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);
		SectionTitle = FText::Format(FTextFormat(LOCTEXT("BoneGroupsSectionTitle", "Bone Groups ({0})")), NeuralMorphEditorModel->GetNeuralMorphModel()->BoneGroups.Num());

		// Update the slate widget.
		RequestTreeRefresh();
	}

	TSharedPtr<SWidget> SNeuralMorphBoneGroupsWidget::CreateContextWidget() const
	{
		return TSharedPtr<SWidget>();
	}

	void SNeuralMorphBoneGroupsTreeRowWidget::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, TSharedRef<FNeuralMorphBoneGroupsTreeElement> InTreeElement, TSharedPtr<SNeuralMorphBoneGroupsWidget> InTreeView)
	{
		WeakTreeElement = InTreeElement;

		STableRow<TSharedPtr<FNeuralMorphBoneGroupsTreeElement>>::Construct
		(
			STableRow<TSharedPtr<FNeuralMorphBoneGroupsTreeElement>>::FArguments()
			.ShowWires(true)
			.Content()
			[
				SNew(STextBlock)
				.Text(this, &SNeuralMorphBoneGroupsTreeRowWidget::GetName)
				.ColorAndOpacity_Lambda
				(
					[this]()
					{
						return WeakTreeElement.IsValid() ? WeakTreeElement.Pin()->TextColor : FSlateColor::UseForeground();
					}
				)
			], 
			OwnerTable
		);
	}

	FText SNeuralMorphBoneGroupsTreeRowWidget::GetName() const
	{
		if (WeakTreeElement.IsValid())
		{
			return FText::FromName(WeakTreeElement.Pin()->Name);
		}
		return FText();
	}

	bool SNeuralMorphBoneGroupsWidget::BroadcastModelPropertyChanged(const FName PropertyName)
	{
		UMLDeformerModel* Model = EditorModel->GetModel();

		FProperty* Property = Model->GetClass()->FindPropertyByName(PropertyName);
		if (Property == nullptr)
		{
			UE_LOGF(LogNeuralMorphModel, Error, "Failed to find property '%ls' in class '%ls'", *PropertyName.ToString(), *Model->GetName());
			return false;
		}

		FPropertyChangedEvent Event(Property, EPropertyChangeType::ValueSet);
		Model->PostEditChangeProperty(Event);
		return true;
	}


	void SNeuralMorphBoneGroupsWidget::Refresh(bool bBroadcastPropertyChanged)
	{
		RefreshTree(bBroadcastPropertyChanged);
	}

	FText SNeuralMorphBoneGroupsWidget::GetSectionTitle() const
	{ 
		return SectionTitle;
	}

	TSharedPtr<UE::MLDeformer::SMLDeformerInputWidget> SNeuralMorphBoneGroupsWidget::GetInputWidget() const
	{
		return InputWidget;
	}

	const TArray<TSharedPtr<FNeuralMorphBoneGroupsTreeElement>>& SNeuralMorphBoneGroupsWidget::GetRootElements() const
	{
		return RootElements;
	}

}	// namespace UE::NeuralMorphModel

#undef LOCTEXT_NAMESPACE
