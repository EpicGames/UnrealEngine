// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNeuralMorphCurveGroupsWidget.h"
#include "NeuralMorphModel.h"
#include "NeuralMorphEditorModel.h"
#include "NeuralMorphInputInfo.h"
#include "SMLDeformerCurvePickerDialog.h"
#include "SNeuralMorphInputWidget.h"
#include "MLDeformerEditorStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Engine/SkeletalMesh.h"
#include "ScopedTransaction.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"
#include "HAL/PlatformApplicationMisc.h"


#define LOCTEXT_NAMESPACE "NeuralMorphCurveGroupsWidget"

namespace UE::NeuralMorphModel
{
	FNeuralMorphCurveGroupsCommands::FNeuralMorphCurveGroupsCommands() 
		: TCommands<FNeuralMorphCurveGroupsCommands>
	(	"Neural Morph Curve Groups",
		NSLOCTEXT("NeuralMorphCurveGroupsWidget", "NeuralMorphCurveGroupsDesc", "Neural Morph Curve Groups"),
		NAME_None,
		FMLDeformerEditorStyle::Get().GetStyleSetName())
	{
	}

	void FNeuralMorphCurveGroupsCommands::RegisterCommands()
	{
		UI_COMMAND(CreateGroup, "Create New Group", "Create a new curve group.", EUserInterfaceActionType::Button, FInputChord(EKeys::Insert));
		UI_COMMAND(DeleteSelectedItems, "Delete Selected Items", "Deletes the selected curves and/or groups.", EUserInterfaceActionType::Button, FInputChord(EKeys::Delete));
		UI_COMMAND(ClearGroups, "Clear All Groups", "Clears the entire list of curve groups.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(AddCurveToGroup, "Add Curves To Group", "Add new curves to the group.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(CopyCurveGroups, "Copy Input Curve Groups List", "Copies all curve groups, so that you can paste them into another model's input curve groups list.", EUserInterfaceActionType::Button, FInputChord(EKeys::C, EModifierKey::Control));
		UI_COMMAND(PasteCurveGroups, "Paste Input Curve Groups List", "Pastes a previously copied curve group list. This will add curve groups to the existing list, not replace it fully.", EUserInterfaceActionType::Button, FInputChord(EKeys::V, EModifierKey::Control));
	}

	void SNeuralMorphCurveGroupsWidget::BindCommands(TSharedPtr<FUICommandList> CommandList)
	{
		const FNeuralMorphCurveGroupsCommands& GroupCommands = FNeuralMorphCurveGroupsCommands::Get();
		CommandList->MapAction(GroupCommands.CreateGroup, FExecuteAction::CreateSP(this, &SNeuralMorphCurveGroupsWidget::OnCreateCurveGroup));
		CommandList->MapAction(GroupCommands.DeleteSelectedItems, FExecuteAction::CreateSP(this, &SNeuralMorphCurveGroupsWidget::OnDeleteSelectedItems));
		CommandList->MapAction(GroupCommands.ClearGroups, FExecuteAction::CreateSP(this, &SNeuralMorphCurveGroupsWidget::OnClearCurveGroups));
		CommandList->MapAction(GroupCommands.AddCurveToGroup, FExecuteAction::CreateSP(this, &SNeuralMorphCurveGroupsWidget::OnAddCurveToGroup));
		CommandList->MapAction(GroupCommands.CopyCurveGroups, FExecuteAction::CreateSP(this, &SNeuralMorphCurveGroupsWidget::OnCopyCurveGroups));
		CommandList->MapAction(GroupCommands.PasteCurveGroups, FExecuteAction::CreateSP(this, &SNeuralMorphCurveGroupsWidget::OnPasteCurveGroups));
	}

	void SNeuralMorphCurveGroupsWidget::Construct(const FArguments& InArgs)
	{
		EditorModel = InArgs._EditorModel;
		InputWidget = InArgs._InputWidget;

		STreeView<TSharedPtr<FNeuralMorphCurveGroupsTreeElement>>::FArguments SuperArgs;
		SuperArgs.TreeItemsSource(&RootElements);
		SuperArgs.SelectionMode(ESelectionMode::Multi);
		SuperArgs.OnGenerateRow(this, &SNeuralMorphCurveGroupsWidget::MakeTableRowWidget);
		SuperArgs.OnGetChildren(this, &SNeuralMorphCurveGroupsWidget::HandleGetChildrenForTree);
		SuperArgs.OnSelectionChanged(this, &SNeuralMorphCurveGroupsWidget::OnSelectionChanged);
		SuperArgs.OnContextMenuOpening(this, &SNeuralMorphCurveGroupsWidget::CreateContextMenuWidget);
		SuperArgs.HighlightParentNodesForSelection(false);
		SuperArgs.AllowInvisibleItemSelection(true);  // Without this we deselect everything when we filter or we collapse.

		STreeView<TSharedPtr<FNeuralMorphCurveGroupsTreeElement>>::Construct(SuperArgs);

		RefreshTree(false);
	}

	void SNeuralMorphCurveGroupsWidget::OnSelectionChanged(TSharedPtr<FNeuralMorphCurveGroupsTreeElement> Selection, ESelectInfo::Type SelectInfo)
	{
		const TSharedPtr<SNeuralMorphInputWidget> NeuralInputWidget = StaticCastSharedPtr<SNeuralMorphInputWidget>(InputWidget);
		if (NeuralInputWidget.IsValid())
		{
			NeuralInputWidget->OnSelectInputCurveGroup(Selection);
		}
	}

	void SNeuralMorphCurveGroupsWidget::OnCopyCurveGroups() const
	{
		const UNeuralMorphModel* NeuralMorphModel = EditorModel->GetNeuralMorphModel();
		if (NeuralMorphModel->GetCurveGroups().IsEmpty())
		{
			return;
		}

		const TSharedRef<FJsonObject> JsonObject = MakeShared<FJsonObject>();
		JsonObject->SetStringField(TEXT("DataId"), TEXT("CurveGroups"));

		// Create each curve group.
		TArray<TSharedPtr<FJsonValue>> CurveGroupsArray;
		for (const FNeuralMorphCurveGroup& CurveGroup : NeuralMorphModel->GetCurveGroups())
		{
			TSharedRef<FJsonObject> JsonCurveroup = MakeShared<FJsonObject>();
			JsonCurveroup->SetStringField(TEXT("GroupName"), CurveGroup.GroupName.ToString());
    
			TArray<TSharedPtr<FJsonValue>> CurveNames;
			for (const FMLDeformerCurveReference& CurveName : CurveGroup.CurveNames)
			{
				if (CurveName.CurveName.IsValid() && !CurveName.CurveName.IsNone())
				{
					CurveNames.Add(MakeShared<FJsonValueString>(CurveName.CurveName.ToString()));
				}
			}
	
			JsonCurveroup->SetArrayField(TEXT("CurveNames"), CurveNames);
			CurveGroupsArray.Add(MakeShared<FJsonValueObject>(JsonCurveroup));
		}

		JsonObject->SetArrayField(TEXT("CurveGroups"), CurveGroupsArray);

		// Convert the JSON to a string and copy it to the clipboard.
		FString OutputString;
		const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);
		if (!FJsonSerializer::Serialize(JsonObject, Writer))
		{
			UE_LOGF(LogMLDeformer, Error, "Failed to serialize JSON on Copy action in the Input Curve Groups widget.");
			return;
		}
		FPlatformApplicationMisc::ClipboardCopy(*OutputString);
	}

	void SNeuralMorphCurveGroupsWidget::OnPasteCurveGroups()
	{
		FString Text;
		FPlatformApplicationMisc::ClipboardPaste(Text);

		TSharedPtr<FJsonObject> JsonObject;
		if (GetInputWidget()->HasValidClipBoardData(TEXT("CurveGroups"), JsonObject) && JsonObject.IsValid())
		{
			const TArray<TSharedPtr<FJsonValue>>* CurveGroupsArray;	
			if (JsonObject->TryGetArrayField(TEXT("CurveGroups"), CurveGroupsArray))
			{
				UNeuralMorphModel* NeuralMorphModel = EditorModel->GetNeuralMorphModel();
				FScopedTransaction Transaction(TEXT("SNeuralMorphCurveGroupsWidget"), LOCTEXT("PasteCurveGroupText", "Paste Curve Groups"), NeuralMorphModel);
				NeuralMorphModel->Modify();
	
				for (const TSharedPtr<FJsonValue>& GroupValue : *CurveGroupsArray)
				{
					TSharedPtr<FJsonObject> GroupObject = GroupValue->AsObject();
					if (!GroupObject.IsValid())
					{
						UE_LOGF(LogNeuralMorphModel, Error, "Skipping invalid group object when pasting curve groups.");
						continue;
					}

					FString GroupNameString;
					if (!GroupObject->TryGetStringField(TEXT("GroupName"), GroupNameString))
					{
						UE_LOGF(LogNeuralMorphModel, Error, "Failed to read curve group name when pasting curve groups.");
						continue;
					}
					
					const TArray<TSharedPtr<FJsonValue>>* CurveArray;
					if (!GroupObject->TryGetArrayField(TEXT("CurveNames"), CurveArray))
					{
						UE_LOGF(LogNeuralMorphModel, Error, "Failed to read curve group curve names array when pasting curve groups.");
						continue;
					}

					// Check if the copied group name is unique or if we have to generate a new one.
					bool bIsUniqueName = true;
					for (const FNeuralMorphCurveGroup& Group : NeuralMorphModel->CurveGroups)
					{
						if (Group.GroupName == GroupNameString)
						{
							bIsUniqueName = false;
							break;
						}
					}
					
					const FName GroupName = bIsUniqueName ? FName(GroupNameString) : EditorModel->GenerateUniqueCurveGroupName();
					FNeuralMorphCurveGroup& CurveGroup = NeuralMorphModel->CurveGroups.AddDefaulted_GetRef();
					CurveGroup.GroupName = GroupName;

					// Add the curves to the newly created group.
					for (const TSharedPtr<FJsonValue>& CurveNameValue : *CurveArray)
					{
						const FString CurveName = CurveNameValue->AsString();
						CurveGroup.CurveNames.AddUnique(FName(CurveName));
					}
				}
			}
			
			RefreshTree(true);
			EditorModel->RebuildEditorMaskInfo();
		}
	}
	
	TSharedPtr<SWidget> SNeuralMorphCurveGroupsWidget::CreateContextMenuWidget() const
	{
		const FNeuralMorphCurveGroupsCommands& Actions = FNeuralMorphCurveGroupsCommands::Get();

		TSharedPtr<FUICommandList> CommandList = StaticCastSharedPtr<SNeuralMorphInputWidget>(InputWidget)->GetCurveGroupsCommandList();
		FMenuBuilder Menu(true, CommandList);

		const TArray<TSharedPtr<FNeuralMorphCurveGroupsTreeElement>> CurSelectedItems = GetSelectedItems();

		Menu.BeginSection("CurveGroupActions", LOCTEXT("CurveGroupActionsHeading", "Curve Group Actions"));
			Menu.AddMenuEntry(Actions.CreateGroup);
		
			if (CurSelectedItems.Num() == 1 && CurSelectedItems[0]->IsGroup())
			{
				Menu.AddMenuEntry(Actions.AddCurveToGroup);
			}

			if (!CurSelectedItems.IsEmpty())
			{			
				Menu.AddMenuEntry(Actions.DeleteSelectedItems);
			}

			if (!GetRootElements().IsEmpty())
			{
				Menu.AddMenuEntry(Actions.ClearGroups);
			}
		Menu.EndSection();

		Menu.BeginSection("CopyPasteActions", LOCTEXT("CopyPasteActionsHeading", "Copy/Paste Actions"));
		{
			if (!RootElements.IsEmpty())
			{
				Menu.AddMenuEntry(Actions.CopyCurveGroups);
			}

			TSharedPtr<FJsonObject> JsonObject;
			if (InputWidget->HasValidClipBoardData(TEXT("CurveGroups"), JsonObject))
			{
				Menu.AddMenuEntry(Actions.PasteCurveGroups);
			}
		}
		Menu.EndSection();

		return Menu.MakeWidget();
	}

	void SNeuralMorphCurveGroupsWidget::OnCreateCurveGroup()
	{
		using namespace UE::MLDeformer;

		USkeletalMesh* SkelMesh = EditorModel->GetModel()->GetSkeletalMesh();
		if (!SkelMesh)
		{
			return;
		}

		UNeuralMorphModel* NeuralMorphModel = EditorModel->GetNeuralMorphModel();
		const UNeuralMorphInputInfo* InputInfo = Cast<const UNeuralMorphInputInfo>(EditorModel->GetEditorInputInfo());
		check(InputInfo);

		TSharedPtr<SMLDeformerCurvePickerDialog> Dialog = 
			SNew(SMLDeformerCurvePickerDialog)
			.SkeletalMesh(SkelMesh)
			.AllowMultiSelect(true)
			.IncludeList(InputInfo->GetCurveNames());

		Dialog->ShowModal();

		const TArray<FName>& CurveNames = Dialog->GetPickedCurveNames();
		if (CurveNames.IsEmpty())
		{
			return;
		}

		FScopedTransaction Transaction(TEXT("SNeuralMorphCurveGroupsWidget"), LOCTEXT("CreateCurveGroupText", "Create Curve Group"), NeuralMorphModel);
		NeuralMorphModel->Modify();

		NeuralMorphModel->CurveGroups.AddDefaulted();
		FNeuralMorphCurveGroup& CurveGroup = NeuralMorphModel->CurveGroups.Last();
		CurveGroup.GroupName = EditorModel->GenerateUniqueCurveGroupName();
		for (const FName CurveName : CurveNames)
		{
			CurveGroup.CurveNames.Add(CurveName);
		}

		EditorModel->UpdateEditorInputInfo();
		RefreshTree(true);
		EditorModel->RebuildEditorMaskInfo();
	}

	void SNeuralMorphCurveGroupsWidget::OnDeleteSelectedItems()
	{
		const TArray<TSharedPtr<FNeuralMorphCurveGroupsTreeElement>> CurSelectedItems = GetSelectedItems();

		TArray<int32> GroupsToRemove;
		TArray<TSharedPtr<FNeuralMorphCurveGroupsTreeElement>> SelectedGroups;
		TArray<TSharedPtr<FNeuralMorphCurveGroupsTreeElement>> SelectedCurves;

		// Check if the selection contains curves and/or groups.
		for (const TSharedPtr<FNeuralMorphCurveGroupsTreeElement>& SelectedItem : CurSelectedItems)
		{
			if (SelectedItem->IsGroup())
			{
				SelectedGroups.Add(SelectedItem);
				GroupsToRemove.Add(SelectedItem->GroupIndex);
			}
			else
			{
				SelectedCurves.Add(SelectedItem);
			}
		}

		UNeuralMorphModel* NeuralMorphModel = EditorModel->GetNeuralMorphModel();

		FScopedTransaction Transaction(TEXT("SNeuralMorphCurveGroupsWidget"), LOCTEXT("RemoveCurveGroupItemsText", "Remove Curve Group Items"), NeuralMorphModel);
		NeuralMorphModel->Modify();

		// Remove all selected curves.
		if (!SelectedCurves.IsEmpty())
		{
			for (const TSharedPtr<FNeuralMorphCurveGroupsTreeElement>& CurveItem : SelectedCurves)
			{
				if (CurveItem.IsValid() && CurveItem->ParentGroup.IsValid())
				{
					TSharedPtr<FNeuralMorphCurveGroupsTreeElement> ParentGroup = CurveItem->ParentGroup.Pin();

					if (CurveItem->Name.IsNone())
					{
						// Remove all the none items.
						NeuralMorphModel->CurveGroups[ParentGroup->GroupIndex].CurveNames.RemoveAll(
							[](const FMLDeformerCurveReference& Item) 
							{
								return Item.CurveName.IsNone();
							});
					}
					else
					{
						NeuralMorphModel->CurveGroups[ParentGroup->GroupIndex].CurveNames.Remove(CurveItem->Name);
					}
				}
			}
		}

		// Sort group indices big to small (back to front).
		if (!GroupsToRemove.IsEmpty())
		{
			GroupsToRemove.Sort([](const int32& A, const int32& B){ return A > B; });

			// Remove the items, back to front.
			for (const int32 Index : GroupsToRemove)
			{
				NeuralMorphModel->CurveGroups.RemoveAt(Index);
			}
		}

		EditorModel->UpdateEditorInputInfo();
		RefreshTree(true);
		EditorModel->RebuildEditorMaskInfo();
	}

	void SNeuralMorphCurveGroupsWidget::OnClearCurveGroups()
	{
		UNeuralMorphModel* NeuralMorphModel = EditorModel->GetNeuralMorphModel();

		FScopedTransaction Transaction(TEXT("SNeuralMorphCurveGroupsWidget"), LOCTEXT("ClearCurveGroupText", "Clear Curve Groups"), NeuralMorphModel);
		NeuralMorphModel->Modify();

		NeuralMorphModel->CurveGroups.Empty();
		EditorModel->UpdateEditorInputInfo();
		RefreshTree(true);
		EditorModel->RebuildEditorMaskInfo();
	}

	void SNeuralMorphCurveGroupsWidget::OnAddCurveToGroup()
	{
		using namespace UE::MLDeformer;

		USkeletalMesh* SkelMesh = EditorModel->GetModel()->GetSkeletalMesh();
		if (!SkelMesh)
		{
			return;
		}

		// Find the group we want to add something to.
		check(GetSelectedItems().Num() == 1);
		const int32 GroupIndex = GetSelectedItems()[0]->GroupIndex;
		check(GroupIndex != INDEX_NONE);
		UNeuralMorphModel* NeuralMorphModel = EditorModel->GetNeuralMorphModel();
		FNeuralMorphCurveGroup& CurveGroup = NeuralMorphModel->CurveGroups[GroupIndex];
		UNeuralMorphInputInfo* InputInfo = Cast<UNeuralMorphInputInfo>(EditorModel->GetEditorInputInfo());
		check(InputInfo);

		// Build the highlighted curve names list.
		TArray<FName> HighlightedCurves;
		HighlightedCurves.Reserve(CurveGroup.CurveNames.Num());
		for (const FMLDeformerCurveReference& CurveRef : CurveGroup.CurveNames)
		{
			HighlightedCurves.Add(CurveRef.CurveName);
		}

		const FLinearColor HighlightColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.InputsWidget.HighlightColor");
		TSharedPtr<SMLDeformerCurvePickerDialog> Dialog = 
			SNew(SMLDeformerCurvePickerDialog)
			.SkeletalMesh(SkelMesh)
			.AllowMultiSelect(true)
			.HighlightCurveNames(HighlightedCurves)
			.HighlightCurveNamesColor(HighlightColor)
			.IncludeList(InputInfo->GetCurveNames());

		Dialog->ShowModal();

		const TArray<FName>& CurveNames = Dialog->GetPickedCurveNames();
		if (!CurveNames.IsEmpty())
		{
			FScopedTransaction Transaction(TEXT("SNeuralMorphCurveGroupsWidget"), LOCTEXT("AddCurveToGroupText", "Add Curves To Group"), NeuralMorphModel);
			NeuralMorphModel->Modify();

			for (const FName CurveName : CurveNames)
			{
				CurveGroup.CurveNames.Add(CurveName);
			}

			EditorModel->UpdateEditorInputInfo();
			RefreshTree(true);
			EditorModel->RebuildEditorMaskInfo();
		}
	}

	void SNeuralMorphCurveGroupsWidget::AddElement(TSharedPtr<FNeuralMorphCurveGroupsTreeElement> Element, TSharedPtr<FNeuralMorphCurveGroupsTreeElement> ParentElement)
	{
		if (!ParentElement)
		{
			RootElements.Add(Element);
		}
		else
		{
			ParentElement->Children.Add(Element);
			Element->ParentGroup = ParentElement->AsWeak();
		}
	}

	TSharedRef<ITableRow> FNeuralMorphCurveGroupsTreeElement::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FNeuralMorphCurveGroupsTreeElement> InTreeElement, TSharedPtr<SNeuralMorphCurveGroupsWidget> InTreeWidget)
	{
		return SNew(SNeuralMorphCurveGroupsTreeRowWidget, InOwnerTable, InTreeElement, InTreeWidget);
	}

	TSharedRef<ITableRow> SNeuralMorphCurveGroupsWidget::MakeTableRowWidget(TSharedPtr<FNeuralMorphCurveGroupsTreeElement> InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return InItem->MakeTreeRowWidget(OwnerTable, InItem.ToSharedRef(), SharedThis(this));
	}

	void SNeuralMorphCurveGroupsWidget::HandleGetChildrenForTree(TSharedPtr<FNeuralMorphCurveGroupsTreeElement> InItem, TArray<TSharedPtr<FNeuralMorphCurveGroupsTreeElement>>& OutChildren)
	{
		OutChildren = InItem.Get()->Children;
	}

	void SNeuralMorphCurveGroupsWidget::UpdateTreeElements()
	{
		RootElements.Reset();

		const FLinearColor ErrorColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.InputsWidget.ErrorColor");

		UNeuralMorphModel* NeuralMorphModel = EditorModel->GetNeuralMorphModel();

		const int32 NumGroups = NeuralMorphModel->GetCurveGroups().Num();
		for (int32 CurveGroupIndex = 0; CurveGroupIndex < NumGroups; ++CurveGroupIndex)
		{
			const FNeuralMorphCurveGroup& CurveGroup = NeuralMorphModel->GetCurveGroups()[CurveGroupIndex];
			bool bGroupHasError = false;

			for (const FMLDeformerCurveReference& CurveRef : CurveGroup.CurveNames)
			{
				if (!CurveRef.CurveName.IsValid() || CurveRef.CurveName.IsNone())
				{
					bGroupHasError = true;
					break;
				}
			}

			// Add the group header.
			TSharedPtr<FNeuralMorphCurveGroupsTreeElement> GroupElement = MakeShared<FNeuralMorphCurveGroupsTreeElement>();
			GroupElement->Name = CurveGroup.GroupName;
			GroupElement->TextColor = bGroupHasError ? FSlateColor(ErrorColor) : FSlateColor::UseForeground();
			GroupElement->GroupIndex = CurveGroupIndex;
			AddElement(GroupElement, nullptr);
			SetItemExpansion(GroupElement, true);

			// Add the items in the group.
			for (int32 CurveIndex = 0; CurveIndex < CurveGroup.CurveNames.Num(); ++CurveIndex)
			{
				const FName CurveName = CurveGroup.CurveNames[CurveIndex].CurveName;
				const bool bCurveHasError = !EditorModel->GetEditorInputInfo()->GetCurveNames().Contains(CurveName);

				TSharedPtr<FNeuralMorphCurveGroupsTreeElement> ItemElement = MakeShared<FNeuralMorphCurveGroupsTreeElement>();
				ItemElement->Name = CurveName;
				ItemElement->TextColor = bCurveHasError ? FSlateColor(ErrorColor) : FSlateColor::UseForeground();
				ItemElement->GroupCurveIndex = CurveIndex;
				AddElement(ItemElement, GroupElement);

				bGroupHasError |= bCurveHasError;
			}

			if (bGroupHasError)
			{
				GroupElement->TextColor = FSlateColor(ErrorColor);
			}
		}
	}

	FReply SNeuralMorphCurveGroupsWidget::OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
	{		
		TSharedPtr<FUICommandList> CommandList = StaticCastSharedPtr<SNeuralMorphInputWidget>(InputWidget)->GetCurveGroupsCommandList();
		if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}

		return STreeView<TSharedPtr<FNeuralMorphCurveGroupsTreeElement>>::OnKeyDown(InGeometry, InKeyEvent);
	}

	void SNeuralMorphCurveGroupsWidget::RefreshTree(bool bBroadcastPropertyChanged)
	{
		if (bBroadcastPropertyChanged)
		{
			UNeuralMorphModel* NeuralMorphModel = EditorModel->GetNeuralMorphModel();
			BroadcastModelPropertyChanged(GET_MEMBER_NAME_CHECKED(UNeuralMorphModel, CurveGroups));
		}

		UpdateTreeElements();

		FNeuralMorphEditorModel* NeuralMorphEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);
		SectionTitle = FText::Format(FTextFormat(LOCTEXT("CurveGroupsSectionTitle", "Curve Groups ({0})")), NeuralMorphEditorModel->GetNeuralMorphModel()->CurveGroups.Num());

		// Update the slate widget.
		RequestTreeRefresh();
	}

	TSharedPtr<SWidget> SNeuralMorphCurveGroupsWidget::CreateContextWidget() const
	{
		return TSharedPtr<SWidget>();
	}

	void SNeuralMorphCurveGroupsTreeRowWidget::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, TSharedRef<FNeuralMorphCurveGroupsTreeElement> InTreeElement, TSharedPtr<SNeuralMorphCurveGroupsWidget> InTreeView)
	{
		WeakTreeElement = InTreeElement;

		STableRow<TSharedPtr<FNeuralMorphCurveGroupsTreeElement>>::Construct
		(
			STableRow<TSharedPtr<FNeuralMorphCurveGroupsTreeElement>>::FArguments()
			.ShowWires(true)
			.Content()
			[
				SNew(STextBlock)
				.Text(this, &SNeuralMorphCurveGroupsTreeRowWidget::GetName)
				.ColorAndOpacity_Lambda
				(
					[this]()
					{
						return WeakTreeElement.IsValid() ? WeakTreeElement.Pin()->TextColor : FSlateColor::UseForeground();
					}
				)
			], 
			OwnerTable
		);
	}

	FText SNeuralMorphCurveGroupsTreeRowWidget::GetName() const
	{
		if (WeakTreeElement.IsValid())
		{
			return FText::FromName(WeakTreeElement.Pin()->Name);
		}
		return FText();
	}

	bool SNeuralMorphCurveGroupsWidget::BroadcastModelPropertyChanged(const FName PropertyName)
	{
		UMLDeformerModel* Model = EditorModel->GetModel();

		FProperty* Property = Model->GetClass()->FindPropertyByName(PropertyName);
		if (Property == nullptr)
		{
			UE_LOGF(LogNeuralMorphModel, Error, "Failed to find property '%ls' in class '%ls'", *PropertyName.ToString(), *Model->GetName());
			return false;
		}

		FPropertyChangedEvent Event(Property, EPropertyChangeType::ValueSet);
		Model->PostEditChangeProperty(Event);
		return true;
	}

	int32 SNeuralMorphCurveGroupsWidget::GetNumSelectedGroups() const
	{
		int32 NumSelectedGroups = 0;
		for (const TSharedPtr<FNeuralMorphCurveGroupsTreeElement>& Item : SelectedItems)
		{
			if (Item->IsGroup())
			{
				NumSelectedGroups++;
			}
		}
		return NumSelectedGroups;
	}

	FText SNeuralMorphCurveGroupsWidget::GetSectionTitle() const
	{ 
		return SectionTitle;
	}

	void SNeuralMorphCurveGroupsWidget::Refresh(bool bBroadcastPropertyChanged)
	{ 
		RefreshTree(bBroadcastPropertyChanged);
	}
	
	TSharedPtr<UE::MLDeformer::SMLDeformerInputWidget> SNeuralMorphCurveGroupsWidget::GetInputWidget() const
	{ 
		return InputWidget;
	}

	const TArray<TSharedPtr<FNeuralMorphCurveGroupsTreeElement>>& SNeuralMorphCurveGroupsWidget::GetRootElements() const
	{
		return RootElements;
	}

}	// namespace UE::NeuralMorphModel

#undef LOCTEXT_NAMESPACE
