// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNeuralMorphInputWidget.h"
#include "NeuralMorphEditorModel.h"
#include "NeuralMorphInputInfo.h"
#include "SNeuralMorphBoneGroupsWidget.h"
#include "SNeuralMorphCurveGroupsWidget.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Dialog/SCustomDialog.h"
#include "MLDeformerEditorStyle.h"
#include "MLDeformerEditorModel.h"
#include "MLDeformerInputInfo.h"
#include "MLDeformerModel.h"
#include "MLDeformerEditorToolkit.h"
#include "SMLDeformerInputCurvesWidget.h"
#include "SMLDeformerInputBonesWidget.h"
#include "SMLDeformerBonePickerDialog.h"
#include "SMLDeformerCurvePickerDialog.h"
#include "SMLDeformerMaskConfigDialog.h"
#include "Engine/SkeletalMesh.h"
#include "MeshDescription.h"
#include "IDetailsView.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "NeuralMorphInputWidget"

namespace UE::NeuralMorphModel
{
	using namespace UE::MLDeformer;

	FNeuralMorphInputWidgetCommands::FNeuralMorphInputWidgetCommands()
		: TCommands<FNeuralMorphInputWidgetCommands>
	(	"Neural Morph Input Widget",
		NSLOCTEXT("NeuralMorphInputWidget", "NeuralMorphInputWidgetDesc", "Neural Morph Input Widget"),
		NAME_None,
		FMLDeformerEditorStyle::Get().GetStyleSetName())
	{
	}

	void FNeuralMorphInputWidgetCommands::RegisterCommands()
	{
		// Bone commands.
		UI_COMMAND(ResetAllBoneMasks, "Reset All Bone Masks", "Reset all masks for every bone in the input list.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(ResetSelectedBoneMasks, "Reset Selected Bone Masks", "Reset the masks for all selected bones.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(ExpandSelectedBoneMasks, "Edit Bone Mask", "Specify which bones to include inside the mask.", EUserInterfaceActionType::Button, FInputChord());

		// Bone group commands.
		UI_COMMAND(ResetAllBoneGroupMasks, "Reset All Bone Group Masks", "Reset all masks for every bone group.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(ResetSelectedBoneGroupMasks, "Reset Selected Bone Group Masks", "Reset the masks for all selected bone groups.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(ExpandSelectedBoneGroupMasks, "Edit Group Mask", "Specify which bones to include inside the mask for this group.", EUserInterfaceActionType::Button, FInputChord());

		UI_COMMAND(ConfigureBoneMask, "Configure Mask", "Configure settings specific to the bone masking.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(ConfigureBoneGroupMask, "Configure Mask", "Configure settings specific to the bone group masking.", EUserInterfaceActionType::Button, FInputChord());
	}

	void SNeuralMorphInputWidget::BindCommands()
	{
		const FNeuralMorphInputWidgetCommands& Commands = FNeuralMorphInputWidgetCommands::Get();

		if (InputBonesWidget.IsValid())
		{
			BonesCommandList->MapAction(Commands.ResetAllBoneMasks, FExecuteAction::CreateSP(this, &SNeuralMorphInputWidget::ResetAllBoneMasks));
			BonesCommandList->MapAction(Commands.ResetSelectedBoneMasks, FExecuteAction::CreateSP(this, &SNeuralMorphInputWidget::ResetSelectedBoneMasks));
			BonesCommandList->MapAction(Commands.ExpandSelectedBoneMasks, FExecuteAction::CreateSP(this, &SNeuralMorphInputWidget::ExpandBoneMasks));
			BonesCommandList->MapAction(Commands.ConfigureBoneMask, FExecuteAction::CreateSP(this, &SNeuralMorphInputWidget::ConfigureBoneMask));
		}

		if (BoneGroupsWidget.IsValid())
		{
			BoneGroupsCommandList->MapAction(Commands.ResetAllBoneGroupMasks, FExecuteAction::CreateSP(this, &SNeuralMorphInputWidget::ResetAllBoneGroupMasks));
			BoneGroupsCommandList->MapAction(Commands.ResetSelectedBoneGroupMasks, FExecuteAction::CreateSP(this, &SNeuralMorphInputWidget::ResetSelectedBoneGroupMasks));
			BoneGroupsCommandList->MapAction(Commands.ExpandSelectedBoneGroupMasks, FExecuteAction::CreateSP(this, &SNeuralMorphInputWidget::ExpandBoneGroupMasks));
			BoneGroupsCommandList->MapAction(Commands.ConfigureBoneGroupMask, FExecuteAction::CreateSP(this, &SNeuralMorphInputWidget::ConfigureBoneGroupMask));
		}
	}

	void SNeuralMorphInputWidget::Construct(const FArguments& InArgs)
	{
		SMLDeformerInputWidget::FArguments SuperArgs;
		SuperArgs.EditorModel(InArgs._EditorModel);

		SMLDeformerInputWidget::Construct(SuperArgs);

		FNeuralMorphEditorModel* NeuralEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);
		if (NeuralEditorModel->GetNeuralMorphModel()->GetModelMode() == ENeuralMorphMode::Local)
		{
			AddSectionSeparator();
			CreateBoneGroupsSection();

			AddSectionSeparator();
			CreateCurveGroupsSection();
		}

		BindCommands();
	}

	void SNeuralMorphInputWidget::Refresh()
	{
		SMLDeformerInputWidget::Refresh();

		if (BoneGroupsWidget.IsValid())
		{
			BoneGroupsWidget->Refresh();
		}

		if (CurveGroupsWidget.IsValid())
		{
			CurveGroupsWidget->Refresh();
		}
	}

	void SNeuralMorphInputWidget::ResetAllBoneMasks()
	{
		USkeletalMesh* SkelMesh = EditorModel->GetModel()->GetSkeletalMesh();
		if (!SkelMesh)
		{
			return;
		}


		FNeuralMorphEditorModel* NeuralEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);

		UNeuralMorphModel* NeuralMorphModel = NeuralEditorModel->GetNeuralMorphModel();		
		FScopedTransaction Transaction(TEXT("SNeuralMorphInputWidget"), LOCTEXT("ResetAllBoneMaskText", "Reset All Bone Masks"), NeuralMorphModel);
		NeuralMorphModel->Modify();

		HierarchyDepth = 1;
		NeuralEditorModel->GenerateBoneMaskInfos(HierarchyDepth);
		NeuralEditorModel->RebuildEditorMaskInfo();

		InputBonesWidget->Refresh();
	}

	void SNeuralMorphInputWidget::ResetSelectedBoneMasks()
	{
		USkeletalMesh* SkelMesh = EditorModel->GetModel()->GetSkeletalMesh();
		if (!SkelMesh)
		{
			return;
		}

		FNeuralMorphEditorModel* NeuralEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);
		UNeuralMorphModel* NeuralMorphModel = NeuralEditorModel->GetNeuralMorphModel();		
		FScopedTransaction Transaction(TEXT("SNeuralMorphInputWidget"), LOCTEXT("ResetSelectedBoneMaskText", "Reset Selected Bone Masks"), NeuralMorphModel);
		NeuralMorphModel->Modify();

		// Get the selected input bones.
		check(InputBonesWidget.IsValid());
		check(InputBonesWidget->GetTreeWidget().IsValid());
		TArray<TSharedPtr<FMLDeformerInputBoneTreeElement>> SelectedItems = InputBonesWidget->GetTreeWidget()->GetSelectedItems();

		UNeuralMorphInputInfo* EditorInputInfo = Cast<UNeuralMorphInputInfo>(NeuralEditorModel->GetEditorInputInfo());
		check(EditorInputInfo);

		// For all bones we have selected.
		HierarchyDepth = 1;
		for (const TSharedPtr<FMLDeformerInputBoneTreeElement>& Item : SelectedItems)
		{
			check(Item.IsValid());
			const FName SelectedInputBoneName = Item->Name;

			// Regenerate the mask info for this bone.
			const int32 BoneIndex = EditorInputInfo->GetBoneNames().Find(SelectedInputBoneName);
			check(BoneIndex != INDEX_NONE);
			NeuralEditorModel->GenerateBoneMaskInfo(BoneIndex, HierarchyDepth);
		}

		// Rebuild the mask buffer, since we modified our mask info.
		NeuralEditorModel->RebuildEditorMaskInfo();

		InputBonesWidget->Refresh();
	}

	void SNeuralMorphInputWidget::ExpandBoneMasks()
	{
		USkeletalMesh* SkelMesh = EditorModel->GetModel()->GetSkeletalMesh();
		if (!SkelMesh)
		{
			return;
		}

		const FLinearColor HighlightColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.InputsWidget.HighlightColor");

		// Get the selected input bones.
		check(InputBonesWidget.IsValid());
		check(InputBonesWidget->GetTreeWidget().IsValid());
		TArray<TSharedPtr<FMLDeformerInputBoneTreeElement>> SelectedItems = InputBonesWidget->GetTreeWidget()->GetSelectedItems();

		// If we only selected one item, we can highlight the bones already in the mask info.
		FNeuralMorphEditorModel* NeuralEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);
		UNeuralMorphInputInfo* EditorInputInfo = Cast<UNeuralMorphInputInfo>(NeuralEditorModel->GetEditorInputInfo());
		check(EditorInputInfo);
		TArray<FName> HighlightedBones;
		if (SelectedItems.Num() == 1)
		{
			check(SelectedItems[0].IsValid());
			const FName BoneName = SelectedItems[0]->Name;
			const FMLDeformerMaskInfo* MaskInfo = NeuralEditorModel->GetNeuralMorphModel()->BoneMaskInfoMap.Find(BoneName);
			if (MaskInfo)
			{
				HighlightedBones = MaskInfo->BoneNames;
			}
		}

		// Show the bone picker dialog.
		TSharedPtr<SMLDeformerBonePickerDialog> Dialog = 
			SNew(SMLDeformerBonePickerDialog)
			.RefSkeleton(&SkelMesh->GetRefSkeleton())
			.AllowMultiSelect(true)
			.HighlightBoneNamesColor(FSlateColor(HighlightColor))
			.HighlightBoneNames(HighlightedBones)
			.InitialSelectedBoneNames(HighlightedBones);

		Dialog->ShowModal();

		// If we picked some bones.
		const TArray<FName>& PickedBoneNames = Dialog->GetPickedBoneNames();
		if (!PickedBoneNames.IsEmpty())
		{	
			UNeuralMorphModel* NeuralMorphModel = NeuralEditorModel->GetNeuralMorphModel();		
			FScopedTransaction Transaction(TEXT("SNeuralMorphInputWidget"), LOCTEXT("EditBoneMaskText", "Edit Bone Mask"), NeuralMorphModel);
			NeuralMorphModel->Modify();

			// For all bones we have selected, add the picked mask bones to the mask info.
			for (const TSharedPtr<FMLDeformerInputBoneTreeElement>& Item : SelectedItems)
			{
				check(Item.IsValid());
				const FName SelectedInputBoneName = Item->Name;

				// Add the picked bone name to the mask info of the selected bone.
				const int32 BoneIndex = EditorInputInfo->GetBoneNames().Find(SelectedInputBoneName);
				check(BoneIndex != INDEX_NONE);
				FMLDeformerMaskInfo* MaskInfo = NeuralMorphModel->BoneMaskInfoMap.Find(SelectedInputBoneName);
				if (MaskInfo == nullptr)
				{
					MaskInfo = &NeuralMorphModel->BoneMaskInfoMap.Add(SelectedInputBoneName, FMLDeformerMaskInfo());
				}

				check(MaskInfo);
				MaskInfo->BoneNames.Reset();
				for (const FName PickedBoneName : PickedBoneNames)
				{
					MaskInfo->BoneNames.AddUnique(PickedBoneName);
				}
			}

			// Rebuild the mask buffer, since we modified our mask info.
			NeuralEditorModel->RebuildEditorMaskInfo();

			InputBonesWidget->Refresh();
		}
	}

	void SNeuralMorphInputWidget::ResetAllBoneGroupMasks()
	{
		USkeletalMesh* SkelMesh = EditorModel->GetModel()->GetSkeletalMesh();
		if (!SkelMesh)
		{
			return;
		}

		FNeuralMorphEditorModel* NeuralEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);
		UNeuralMorphModel* NeuralMorphModel = NeuralEditorModel->GetNeuralMorphModel();
		FScopedTransaction Transaction(TEXT("SNeuralMorphInputWidget"), LOCTEXT("ConfigureBoneMaskText", "Configure Bone Mask"), NeuralMorphModel);
		NeuralMorphModel->Modify();
		
		HierarchyDepth = 1;
		NeuralEditorModel->GenerateBoneGroupMaskInfos(HierarchyDepth);
		NeuralEditorModel->RebuildEditorMaskInfo();

		InputBonesWidget->Refresh();
	}

	void SNeuralMorphInputWidget::ResetSelectedBoneGroupMasks()
	{
		USkeletalMesh* SkelMesh = EditorModel->GetModel()->GetSkeletalMesh();
		if (!SkelMesh)
		{
			return;
		}

		// Get the selected bone groups.
		check(BoneGroupsWidget.IsValid());
		TArray<TSharedPtr<FNeuralMorphBoneGroupsTreeElement>> SelectedItems = BoneGroupsWidget->GetSelectedItems();

		FNeuralMorphEditorModel* NeuralEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);
		UNeuralMorphInputInfo* EditorInputInfo = Cast<UNeuralMorphInputInfo>(NeuralEditorModel->GetEditorInputInfo());
		check(EditorInputInfo);

		UNeuralMorphModel* NeuralMorphModel = NeuralEditorModel->GetNeuralMorphModel();
		FScopedTransaction Transaction(TEXT("SNeuralMorphInputWidget"), LOCTEXT("ResetSelectedBoneGroupMask", "Reset Selected Bone Group Mask"), NeuralMorphModel);
		NeuralMorphModel->Modify();

		// For all bone groups we have selected, reset the mask info.
		HierarchyDepth = 1;
		for (const TSharedPtr<FNeuralMorphBoneGroupsTreeElement>& Item : SelectedItems)
		{
			check(Item.IsValid());
			if (Item->IsGroup())
			{
				const int32 BoneGroupIndex = Item->GroupIndex;
				check(BoneGroupIndex != INDEX_NONE);
				NeuralEditorModel->GenerateBoneGroupMaskInfo(BoneGroupIndex, HierarchyDepth);
			}
		}

		// Rebuild the mask buffer, since we modified our mask info.
		NeuralEditorModel->RebuildEditorMaskInfo();

		InputBonesWidget->Refresh();
	}

	void SNeuralMorphInputWidget::ExpandBoneGroupMasks()
	{
		USkeletalMesh* SkelMesh = EditorModel->GetModel()->GetSkeletalMesh();
		if (!SkelMesh)
		{
			return;
		}

		const FLinearColor HighlightColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.InputsWidget.HighlightColor");

		// Get the selected bone groups.
		check(BoneGroupsWidget.IsValid());
		TArray<TSharedPtr<FNeuralMorphBoneGroupsTreeElement>> SelectedItems = BoneGroupsWidget->GetSelectedItems();

		// Calculate the number of selected groups.
		int32 NumSelectedGroups = 0;
		TSharedPtr<FNeuralMorphBoneGroupsTreeElement> LastSelectedGroup;
		for (const TSharedPtr<FNeuralMorphBoneGroupsTreeElement>& Item : SelectedItems)
		{
			if (Item->IsGroup())
			{
				LastSelectedGroup = Item;
				NumSelectedGroups++;
			}
		}

		// If we only selected one item, we can highlight the bones already in the mask info.
		FNeuralMorphEditorModel* NeuralEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);
		UNeuralMorphModel* NeuralMorphModel = NeuralEditorModel->GetNeuralMorphModel();
		UNeuralMorphInputInfo* EditorInputInfo = Cast<UNeuralMorphInputInfo>(NeuralEditorModel->GetEditorInputInfo());
		check(EditorInputInfo);
		TArray<FName> HighlightedBones;
		if (LastSelectedGroup.IsValid() && NumSelectedGroups == 1)
		{
			check(LastSelectedGroup->IsGroup());
			const int32 GroupIndex = LastSelectedGroup->GroupIndex;
			check(GroupIndex!= INDEX_NONE);
			const FMLDeformerMaskInfo* MaskInfo = NeuralMorphModel->BoneGroupMaskInfoMap.Find(LastSelectedGroup->Name);
			if (MaskInfo)
			{
				HighlightedBones = MaskInfo->BoneNames;
			}
		}

		// Show the bone picker dialog.
		TSharedPtr<SMLDeformerBonePickerDialog> Dialog = 
			SNew(SMLDeformerBonePickerDialog)
			.RefSkeleton(&SkelMesh->GetRefSkeleton())
			.AllowMultiSelect(true)
			.HighlightBoneNamesColor(FSlateColor(HighlightColor))
			.HighlightBoneNames(HighlightedBones)
			.InitialSelectedBoneNames(HighlightedBones);

		Dialog->ShowModal();

		// If we picked some bones.
		const TArray<FName>& PickedBoneNames = Dialog->GetPickedBoneNames();
		if (!PickedBoneNames.IsEmpty())
		{	
			FScopedTransaction Transaction(TEXT("SNeuralMorphInputWidget"), LOCTEXT("EditBoneGroupMaskText", "Edit Bone Group Mask"), NeuralMorphModel);
			NeuralMorphModel->Modify();

			// For all groups we selected.
			for (const TSharedPtr<FNeuralMorphBoneGroupsTreeElement>& Item : SelectedItems)
			{
				check(Item.IsValid());

				if (!Item->IsGroup())
				{
					continue;
				}

				// Add the picked bone name to the mask info of the selected group.
				FMLDeformerMaskInfo* MaskInfo = NeuralMorphModel->BoneGroupMaskInfoMap.Find(Item->Name);
				if (MaskInfo == nullptr)
				{
					MaskInfo = &NeuralMorphModel->BoneGroupMaskInfoMap.Add(Item->Name, FMLDeformerMaskInfo());
				}
				check(MaskInfo);
				MaskInfo->BoneNames.Reset();
				for (const FName PickedBoneName : PickedBoneNames)
				{
					MaskInfo->BoneNames.AddUnique(PickedBoneName);
				}
			}

			// Rebuild the mask buffer, since we modified our mask info.
			NeuralEditorModel->RebuildEditorMaskInfo();

			InputBonesWidget->Refresh();
		}
	}

	void SNeuralMorphInputWidget::AddInputBonesMenuItems(FMenuBuilder& MenuBuilder)
	{
		UNeuralMorphModel* NeuralMorphModel = Cast<UNeuralMorphModel>(EditorModel->GetModel());
		if (NeuralMorphModel->GetModelMode() != ENeuralMorphMode::Local)
		{
			return;
		}

		MenuBuilder.BeginSection("BoneMaskActions", LOCTEXT("BoneMaskActionsHeading", "Bone Masks"));
		{
			const FNeuralMorphInputWidgetCommands& Commands = FNeuralMorphInputWidgetCommands::Get();

			// Check if we have only generated masks or not.
			bool bHasNonGeneratedMask = false;
			const int32 NumSelectedItems = InputBonesWidget->GetTreeWidget()->GetNumItemsSelected();
			const TArray<TSharedPtr<FMLDeformerInputBoneTreeElement>> SelectedItems = InputBonesWidget->GetTreeWidget()->GetSelectedItems();
			for (const TSharedPtr<FMLDeformerInputBoneTreeElement>& Item : SelectedItems)
			{
				if (!Item.IsValid())
				{
					continue;
				}

				FMLDeformerMaskInfo* MaskInfo = NeuralMorphModel->BoneMaskInfoMap.Find(Item->Name);
				if (MaskInfo && MaskInfo->MaskMode != EMLDeformerMaskingMode::Generated)
				{
					bHasNonGeneratedMask = true;
				}
			}

			if (NumSelectedItems > 0)
			{
				if (!bHasNonGeneratedMask)
				{
					MenuBuilder.AddMenuEntry(Commands.ExpandSelectedBoneMasks);
					MenuBuilder.AddMenuEntry(Commands.ResetSelectedBoneMasks);
				}

				if (NumSelectedItems == 1)
				{
					if (!bHasNonGeneratedMask)
					{
						MenuBuilder.AddMenuSeparator();
					}
					MenuBuilder.AddMenuEntry(Commands.ConfigureBoneMask);
				}
			}
			MenuBuilder.EndSection();
		}

	}

	void SNeuralMorphInputWidget::AddInputBonesPlusIconMenuItems(FMenuBuilder& MenuBuilder)
	{
		UNeuralMorphModel* NeuralMorphModel = Cast<UNeuralMorphModel>(EditorModel->GetModel());
		if (NeuralMorphModel->GetModelMode() != ENeuralMorphMode::Local)
		{
			return;
		}

		MenuBuilder.BeginSection("BoneMaskPlusIconActions", LOCTEXT("BoneMaskActionsPlusIconHeading", "Bone Masks"));
		{
			const FNeuralMorphInputWidgetCommands& Commands = FNeuralMorphInputWidgetCommands::Get();
			const int32 NumItems = InputBonesWidget->GetTreeWidget()->GetNumItemsBeingObserved();
			if (NumItems > 0)
			{
				MenuBuilder.AddMenuEntry(Commands.ResetAllBoneMasks);
			}
		}
		MenuBuilder.EndSection();
	}

	void SNeuralMorphInputWidget::OnClearInputBones()
	{
		FNeuralMorphEditorModel* NeuralEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);
		UNeuralMorphModel* NeuralMorphModel = NeuralEditorModel->GetNeuralMorphModel();
		NeuralMorphModel->BoneMaskInfoMap.Empty();
		NeuralMorphModel->BoneGroupMaskInfoMap.Empty();
		NeuralEditorModel->RebuildEditorMaskInfo();
	}

	void SNeuralMorphInputWidget::OnDeleteInputBones(const TArray<FName>& Names)
	{
		FNeuralMorphEditorModel* NeuralEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);
		UNeuralMorphModel* NeuralMorphModel = NeuralEditorModel->GetNeuralMorphModel();

		for (const FName Name : Names)
		{
			NeuralMorphModel->BoneMaskInfoMap.Remove(Name);

			// Remove the bone from any bone group masks.
			for (auto& MaskInfo : NeuralMorphModel->BoneGroupMaskInfoMap)
			{
				MaskInfo.Value.BoneNames.Remove(Name);
			}
		}

		NeuralEditorModel->RebuildEditorMaskInfo();
	}

	void SNeuralMorphInputWidget::OnDeleteInputCurves(const TArray<FName>& Names)
	{
		FNeuralMorphEditorModel* NeuralEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);
		UNeuralMorphModel* NeuralMorphModel = NeuralEditorModel->GetNeuralMorphModel();
		NeuralEditorModel->RebuildEditorMaskInfo();
	}

	void SNeuralMorphInputWidget::AddInputBoneGroupsMenuItems(FMenuBuilder& MenuBuilder)
	{
		UNeuralMorphModel* NeuralMorphModel = Cast<UNeuralMorphModel>(EditorModel->GetModel());
		if (NeuralMorphModel->GetModelMode() != ENeuralMorphMode::Local)
		{
			return;
		}

		MenuBuilder.BeginSection("BoneGroupMaskActions", LOCTEXT("BoneGroupMaskActionsHeading", "Bone Group Masks"));
		{
			const FNeuralMorphInputWidgetCommands& Commands = FNeuralMorphInputWidgetCommands::Get();
			if (BoneGroupsWidget->GetNumSelectedGroups() > 0)
			{
				TSharedPtr<FNeuralMorphBoneGroupsTreeElement> LastSelectedGroup;
				for (const TSharedPtr<FNeuralMorphBoneGroupsTreeElement>& Item : BoneGroupsWidget->GetSelectedItems())
				{
					if (Item->IsGroup())
					{
						LastSelectedGroup = Item;
					}
				}
				check(LastSelectedGroup.IsValid());

				bool bIsNonGeneratedMask = false;
				FMLDeformerMaskInfo* MaskInfo = NeuralMorphModel->BoneGroupMaskInfoMap.Find(LastSelectedGroup->Name);
				if (MaskInfo && MaskInfo->MaskMode != EMLDeformerMaskingMode::Generated)
				{
					bIsNonGeneratedMask = true;
				}

				if (BoneGroupsWidget->GetNumSelectedGroups() == 1)
				{
					if (!bIsNonGeneratedMask)
					{
						MenuBuilder.AddMenuEntry(Commands.ExpandSelectedBoneGroupMasks);
						MenuBuilder.AddMenuEntry(Commands.ResetSelectedBoneGroupMasks);
						MenuBuilder.AddMenuSeparator();
					}

					MenuBuilder.AddMenuEntry(Commands.ConfigureBoneGroupMask);
				}
			}
		}
		MenuBuilder.EndSection();
	}

	void SNeuralMorphInputWidget::AddInputBoneGroupsPlusIconMenuItems(FMenuBuilder& MenuBuilder)
	{
		UNeuralMorphModel* NeuralMorphModel = Cast<UNeuralMorphModel>(EditorModel->GetModel());
		if (NeuralMorphModel->GetModelMode() != ENeuralMorphMode::Local)
		{
			return;
		}

		MenuBuilder.BeginSection("BoneGroupMaskPlusIconActions", LOCTEXT("BoneGroupMaskPlusIconActionsHeading", "Bone Group Masks"));
		{
			const FNeuralMorphInputWidgetCommands& Commands = FNeuralMorphInputWidgetCommands::Get();
			const int32 NumItems = BoneGroupsWidget->GetNumItemsBeingObserved();
			if (NumItems > 0)
			{
				MenuBuilder.AddMenuEntry(Commands.ResetAllBoneGroupMasks);
			}
		}
		MenuBuilder.EndSection();
	}

	FReply SNeuralMorphInputWidget::ShowBoneGroupsManageContextMenu()
	{
		const FNeuralMorphBoneGroupsCommands& Actions = FNeuralMorphBoneGroupsCommands::Get();
		FMenuBuilder Menu(true, BoneGroupsCommandList);

		Menu.BeginSection("BoneGroupManagementActions", LOCTEXT("BoneGroupManagementActionsHeading", "Bone Group Management"));
		{
			Menu.AddMenuEntry(Actions.CreateGroup);

			FNeuralMorphEditorModel* NeuralEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);
			UNeuralMorphInputInfo* InputInfo = Cast<UNeuralMorphInputInfo>(EditorModel->GetEditorInputInfo());
			if (!InputInfo->GetBoneGroups().IsEmpty())
			{
				Menu.AddMenuEntry(Actions.ClearGroups);
			}
		}
		Menu.EndSection();

		AddInputBoneGroupsPlusIconMenuItems(Menu);

		FSlateApplication::Get().PushMenu(
			this->AsShared(),
			FWidgetPath(),
			Menu.MakeWidget(),
			FSlateApplication::Get().GetCursorPos(),
			FPopupTransitionEffect(FPopupTransitionEffect::TopMenu));

		return FReply::Handled();
	}

	void SNeuralMorphInputWidget::CreateBoneGroupsSection()
	{
		BoneGroupsCommandList = MakeShared<FUICommandList>();

		FNeuralMorphEditorModel* NeuralEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);

		const TSharedPtr<SWidget> MainWidget =
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.MinHeight(75.0f)
			[
				SAssignNew(BoneGroupsWidget, SNeuralMorphBoneGroupsWidget)
					.EditorModel(NeuralEditorModel)
					.InputWidget(SharedThis(this))
			];

		FSectionInfo SectionInfo;
		SectionInfo.SectionTitle = TAttribute<FText>::CreateSP(BoneGroupsWidget.Get(), &SNeuralMorphBoneGroupsWidget::GetSectionTitle);
		SectionInfo.PlusButtonPressed = FOnClicked::CreateSP(this, &SNeuralMorphInputWidget::ShowBoneGroupsManageContextMenu);
		SectionInfo.PlusButtonTooltip = LOCTEXT("BonesGroupsPlusButtonTooltip", "Manage bone groups.");
		AddSection(MainWidget, SectionInfo);

		BoneGroupsWidget->BindCommands(BoneGroupsCommandList);
	}

	FReply SNeuralMorphInputWidget::ShowCurveGroupsManageContextMenu()
	{
		const FNeuralMorphCurveGroupsCommands& Actions = FNeuralMorphCurveGroupsCommands::Get();

		FMenuBuilder Menu(true, CurveGroupsCommandList);

		Menu.BeginSection("CurveGroupManagementActions", LOCTEXT("CurveGroupManagementActionsHeading", "Curve Group Management"));
		{
			Menu.AddMenuEntry(Actions.CreateGroup);

			FNeuralMorphEditorModel* NeuralEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);
			UNeuralMorphInputInfo* InputInfo = Cast<UNeuralMorphInputInfo>(EditorModel->GetEditorInputInfo());
			if (!InputInfo->GetCurveGroups().IsEmpty())
			{
				Menu.AddMenuEntry(Actions.ClearGroups);
			}
		}
		Menu.EndSection();

		FSlateApplication::Get().PushMenu(
			AsShared(),
			FWidgetPath(),
			Menu.MakeWidget(),
			FSlateApplication::Get().GetCursorPos(),
			FPopupTransitionEffect(FPopupTransitionEffect::TopMenu));

		return FReply::Handled();
	}


	void SNeuralMorphInputWidget::CreateCurveGroupsSection()
	{
		CurveGroupsCommandList = MakeShared<FUICommandList>();

		FNeuralMorphEditorModel* NeuralEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);

		const TSharedPtr<SWidget> MainWidget =
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.MinHeight(75.0f)
			[
				SAssignNew(CurveGroupsWidget, SNeuralMorphCurveGroupsWidget)
					.EditorModel(NeuralEditorModel)
					.InputWidget(SharedThis(this))
			];

		FSectionInfo SectionInfo;
		SectionInfo.SectionTitle = TAttribute<FText>::CreateSP(CurveGroupsWidget.Get(), &SNeuralMorphCurveGroupsWidget::GetSectionTitle);
		SectionInfo.PlusButtonPressed = FOnClicked::CreateSP(this, &SNeuralMorphInputWidget::ShowCurveGroupsManageContextMenu);
		SectionInfo.PlusButtonTooltip = LOCTEXT("CurveGroupsPlusButtonTooltip", "Manage curve groups.");
		AddSection(MainWidget, SectionInfo);

		CurveGroupsWidget->BindCommands(CurveGroupsCommandList);
	}

	void SNeuralMorphInputWidget::OnSelectInputBone(FName BoneName)
	{
		if (!BoneName.IsNone())
		{			
			FNeuralMorphEditorModel* NeuralEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);
			const int32 MaskVizItemIndex = NeuralEditorModel->GetEditorInputInfo()->GetBoneNames().Find(BoneName);
			NeuralEditorModel->SetMaskVisualizationItemIndex(MaskVizItemIndex);
			ClearSelectionForAllWidgetsExceptThis(InputBonesWidget->GetTreeWidget());
		}
	}

	void SNeuralMorphInputWidget::OnSelectInputCurve(FName CurveName)
	{
		if (!CurveName.IsNone())
		{
			FNeuralMorphEditorModel* NeuralEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);
			const UNeuralMorphInputInfo* InputInfo = Cast<const UNeuralMorphInputInfo>(NeuralEditorModel->GetEditorInputInfo());
			int32 MaskVizItemIndex = InputInfo->GetCurveNames().Find(CurveName);
			if (MaskVizItemIndex != INDEX_NONE)
			{
				MaskVizItemIndex += InputInfo->GetBoneNames().Num();
			}
			else
			{
				MaskVizItemIndex = INDEX_NONE;
			}
			NeuralEditorModel->SetMaskVisualizationItemIndex(MaskVizItemIndex);
			ClearSelectionForAllWidgetsExceptThis(InputCurvesWidget->GetListWidget());
		}
	}

	void SNeuralMorphInputWidget::OnSelectInputBoneGroup(TSharedPtr<FNeuralMorphBoneGroupsTreeElement> Element)
	{
		if (!Element.IsValid())
		{
			return;
		}

		FNeuralMorphEditorModel* NeuralEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);
		const UNeuralMorphInputInfo* InputInfo = Cast<const UNeuralMorphInputInfo>(NeuralEditorModel->GetEditorInputInfo());

		const FName GroupName = Element->IsGroup() ? Element->Name : Element->ParentGroup.Pin()->Name;
		int32 MaskVizItemIndex = INDEX_NONE;
		if (GroupName.IsValid() && !GroupName.IsNone())
		{
			// Find the group to visualize the mask for.
			// Do this based on the group name.
			for (int32 Index = 0; Index < InputInfo->GetBoneGroups().Num(); ++Index)
			{
				const FNeuralMorphBoneGroup& BoneGroup = InputInfo->GetBoneGroups()[Index];
				if (BoneGroup.GroupName == GroupName)
				{
					MaskVizItemIndex = InputInfo->GetBoneNames().Num() + InputInfo->GetCurveNames().Num() + Index;
					break;
				}
			}
		}

		NeuralEditorModel->SetMaskVisualizationItemIndex(MaskVizItemIndex);
		ClearSelectionForAllWidgetsExceptThis(BoneGroupsWidget);
	}

	void SNeuralMorphInputWidget::OnSelectInputCurveGroup(TSharedPtr<FNeuralMorphCurveGroupsTreeElement> Element)
	{
		if (!Element.IsValid())
		{
			return;
		}

		FNeuralMorphEditorModel* NeuralEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);
		UNeuralMorphInputInfo* InputInfo = Cast<UNeuralMorphInputInfo>(NeuralEditorModel->GetEditorInputInfo());

		const FName GroupName = Element->IsGroup() ? Element->Name : Element->ParentGroup.Pin()->Name;
		int32 MaskVizItemIndex = INDEX_NONE;
		if (GroupName.IsValid() && !GroupName.IsNone())
		{
			// Find the group to visualize the mask for.
			// Do this based on the group name.
			for (int32 Index = 0; Index < InputInfo->GetCurveGroups().Num(); ++Index)
			{
				const FNeuralMorphCurveGroup& CurveGroup = InputInfo->GetCurveGroups()[Index];
				if (CurveGroup.GroupName == GroupName)
				{
					MaskVizItemIndex = InputInfo->GetBoneNames().Num() + InputInfo->GetCurveNames().Num() + InputInfo->GetBoneGroups().Num() + Index;
					break;
				}
			}
		}

		NeuralEditorModel->SetMaskVisualizationItemIndex(MaskVizItemIndex);
		ClearSelectionForAllWidgetsExceptThis(CurveGroupsWidget);
	}

	void SNeuralMorphInputWidget::ClearSelectionForAllWidgetsExceptThis(TSharedPtr<SWidget> ExceptThisWidget)
	{
		SMLDeformerInputWidget::ClearSelectionForAllWidgetsExceptThis(ExceptThisWidget);

		if (BoneGroupsWidget.IsValid() && ExceptThisWidget != BoneGroupsWidget)
		{
			BoneGroupsWidget->ClearSelection();
		}

		if (CurveGroupsWidget.IsValid() && ExceptThisWidget != CurveGroupsWidget)
		{
			CurveGroupsWidget->ClearSelection();
		}
	}

	TSharedPtr<SWidget> SNeuralMorphInputWidget::GetExtraBonePickerWidget()
	{
		const FText Tooltip = LOCTEXT("HierarchyDepthTooltip", 
				"The hierarchy depth represents how many bones up and down the hierarchy to include in the mask.\n" \
				"A value of 1 will generate a mask that includes the parent and child bones.\n" \
				"A value of 2 will generate a mask that includes the parent and child bones, as well as the parent and child bones of those.\n" \
				"You want to make sure the mask is never too small, as that will lead to visual errors.\n" \
				"The disadvantage of a too large mask is that you can get deformations in unwanted areas and it can use more memory at runtime.");

		HierarchyDepth = 1;

		return 
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("MaskHierarchyDepthLabel", "Mask Hierarchy Depth:"))
				.ToolTipText(Tooltip)
			]
			+SHorizontalBox::Slot()
			.MaxWidth(50)
			.HAlign(HAlign_Right)
			.Padding(2.0f, 0.0f)
			.AutoWidth()
			[
				SNew(SNumericEntryBox<int32>)
				.MinDesiredValueWidth(50)
				.MinValue(1)
				.MaxValue(100)
				.MinSliderValue(1)
				.MaxSliderValue(100)
				.ToolTipText(Tooltip)
				.Value_Lambda([this]() { return HierarchyDepth; })
				.OnValueChanged(SNumericEntryBox<int32>::FOnValueChanged::CreateLambda(
					[this](int32 Value)
					{
						HierarchyDepth = Value;
					}))
			];
	}

	void SNeuralMorphInputWidget::OnAddInputBones(const TArray<FName>& Names)
	{
		FNeuralMorphEditorModel* NeuralEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);
		for (const FName BoneName : Names)
		{
			const int32 InputBoneIndex = EditorModel->GetEditorInputInfo()->GetBoneNames().Find(BoneName);
			if (InputBoneIndex != INDEX_NONE)
			{
				NeuralEditorModel->GenerateBoneMaskInfo(InputBoneIndex, HierarchyDepth);
			}
		}
		if (BoneGroupsWidget.IsValid())
		{
			BoneGroupsWidget->Refresh(true);
		}
		NeuralEditorModel->RebuildEditorMaskInfo();
	}

	void SNeuralMorphInputWidget::OnAddInputCurves(const TArray<FName>& Names)
	{
		FNeuralMorphEditorModel* NeuralEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);
		if (CurveGroupsWidget.IsValid())
		{
			CurveGroupsWidget->Refresh(true);
		}
		NeuralEditorModel->RebuildEditorMaskInfo();
	}

	void SNeuralMorphInputWidget::OnAddAnimatedBones()
	{
		FNeuralMorphEditorModel* NeuralEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);
		NeuralEditorModel->RebuildEditorMaskInfo();
	}

	void SNeuralMorphInputWidget::OnAddAnimatedCurves()
	{
		FNeuralMorphEditorModel* NeuralEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);
		NeuralEditorModel->RebuildEditorMaskInfo();
	}

	void SNeuralMorphInputWidget::ConfigureBoneMask()
	{
		using namespace UE::MLDeformer;

		// Get the selected input bones.
		check(InputBonesWidget.IsValid());
		check(InputBonesWidget->GetTreeWidget().IsValid());
		TArray<TSharedPtr<FMLDeformerInputBoneTreeElement>> SelectedItems = InputBonesWidget->GetTreeWidget()->GetSelectedItems();
		check(SelectedItems.Num() == 1);	// This only works for single selection.

		const FName BoneName = SelectedItems[0]->Name;
		UNeuralMorphModel* NeuralModel = Cast<UNeuralMorphModel>(EditorModel->GetModel());
		FMLDeformerMaskInfo NeuralMaskInfo;
		if (NeuralModel->BoneMaskInfoMap.Contains(BoneName))
		{
			NeuralMaskInfo = *NeuralModel->BoneMaskInfoMap.Find(BoneName);
		}

		FScopedTransaction Transaction(TEXT("SNeuralMorphInputWidget"), LOCTEXT("ConfigureBoneMaskText", "Configure Bone Mask"), NeuralModel);
		NeuralModel->Modify();

		// Show the mask configuration dialog.
		TSharedPtr<SMLDeformerMaskConfigDialog> Dialog = SNew(SMLDeformerMaskConfigDialog, EditorModel)
			.InitialMaskInfo(NeuralMaskInfo)
			.OnSetNewVertexAttributeValues_Lambda([this, BoneName, NeuralModel](TVertexAttributesRef<float> AttributeRef)
			{
				// When a new vertex attribute is created in the mask config dialog, we want to initialize it with a generated mask.
				FNeuralMorphEditorModel* NeuralEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);
				UNeuralMorphInputInfo* InputInfo = Cast<UNeuralMorphInputInfo>(NeuralEditorModel->GetEditorInputInfo());
				USkeletalMesh* SkeletalMesh = NeuralModel->GetSkeletalMesh();
				if (InputInfo && SkeletalMesh)
				{
					FMLDeformerMaskInfo* MaskInfo = NeuralModel->BoneMaskInfoMap.Find(BoneName);
					check(MaskInfo);
					NeuralEditorModel->ApplyGeneratedMaskToVertexAttributes(SkeletalMesh, *MaskInfo, AttributeRef);
				}
			});
				
		const int32 ReturnCode = Dialog->ShowModal();
		if (ReturnCode == 0) // OK was pressed.
		{
			// Create the mask info if needed.
			if (!NeuralModel->BoneMaskInfoMap.Contains(BoneName))
			{
				NeuralModel->BoneMaskInfoMap.Add(BoneName, Dialog->GetMaskInfo());
			}
			else
			{
				*NeuralModel->BoneMaskInfoMap.Find(BoneName) = Dialog->GetMaskInfo();
			}

			FNeuralMorphEditorModel* NeuralEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);
			NeuralEditorModel->RebuildEditorMaskInfo();
		}
	}

	void SNeuralMorphInputWidget::ConfigureBoneGroupMask()
	{
		using namespace UE::MLDeformer;

		USkeletalMesh* SkelMesh = EditorModel->GetModel()->GetSkeletalMesh();
		if (!SkelMesh)
		{
			return;
		}

		// Get the selected bone groups.
		check(BoneGroupsWidget.IsValid());
		TArray<TSharedPtr<FNeuralMorphBoneGroupsTreeElement>> SelectedItems = BoneGroupsWidget->GetSelectedItems();

		// Calculate the number of selected groups.
		int32 NumSelectedGroups = 0;
		TSharedPtr<FNeuralMorphBoneGroupsTreeElement> LastSelectedGroup;
		for (const TSharedPtr<FNeuralMorphBoneGroupsTreeElement>& Item : SelectedItems)
		{
			if (Item->IsGroup())
			{
				LastSelectedGroup = Item;
				NumSelectedGroups++;
			}
		}

		// If we only selected one item, we can highlight the bones already in the mask info.
		FNeuralMorphEditorModel* NeuralEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);
		UNeuralMorphModel* NeuralMorphModel = NeuralEditorModel->GetNeuralMorphModel();
		UNeuralMorphInputInfo* EditorInputInfo = Cast<UNeuralMorphInputInfo>(NeuralEditorModel->GetEditorInputInfo());
		check(EditorInputInfo);

		FMLDeformerMaskInfo* GroupMaskInfo = nullptr;
		if (LastSelectedGroup.IsValid() && NumSelectedGroups == 1)
		{
			check(LastSelectedGroup->IsGroup());
			const int32 GroupIndex = LastSelectedGroup->GroupIndex;
			check(GroupIndex!= INDEX_NONE);
			GroupMaskInfo = NeuralMorphModel->BoneGroupMaskInfoMap.Find(LastSelectedGroup->Name);
			if (!GroupMaskInfo)
			{
				return;
			}
		}
		else
		{
			return;
		}

		check(GroupMaskInfo);
		const FName GroupName = LastSelectedGroup->Name;

		FScopedTransaction Transaction(TEXT("SNeuralMorphInputWidget"), LOCTEXT("ConfigureBoneGroupMaskText", "Configure Bone Group Mask"), NeuralMorphModel);
		NeuralMorphModel->Modify();

		// Show the mask configuration dialog.
		TSharedPtr<SMLDeformerMaskConfigDialog> Dialog = SNew(SMLDeformerMaskConfigDialog, EditorModel)
			.InitialMaskInfo(*GroupMaskInfo)
			.OnSetNewVertexAttributeValues_Lambda([this, GroupMaskInfo](TVertexAttributesRef<float> AttributeRef)
			{
				// When a new vertex attribute is created in the mask config dialog, we want to initialize it with a generated mask.
				FNeuralMorphEditorModel* NeuralEditorModel = static_cast<FNeuralMorphEditorModel*>(EditorModel);
				UNeuralMorphInputInfo* InputInfo = Cast<UNeuralMorphInputInfo>(NeuralEditorModel->GetEditorInputInfo());
				USkeletalMesh* SkeletalMesh = EditorModel->GetModel()->GetSkeletalMesh();
				if (InputInfo && SkeletalMesh)
				{
					NeuralEditorModel->ApplyGeneratedMaskToVertexAttributes(SkeletalMesh, *GroupMaskInfo, AttributeRef);
				}
			});
				
		const int32 ReturnCode = Dialog->ShowModal();
		if (ReturnCode == 0) // OK was pressed.
		{
			// Create the mask info if needed.
			UNeuralMorphModel* NeuralModel = Cast<UNeuralMorphModel>(EditorModel->GetModel());
			if (!NeuralModel->BoneGroupMaskInfoMap.Contains(GroupName))
			{
				NeuralModel->BoneGroupMaskInfoMap.Add(GroupName, Dialog->GetMaskInfo());
			}
			else
			{
				*NeuralModel->BoneGroupMaskInfoMap.Find(GroupName) = Dialog->GetMaskInfo();
			}

			NeuralEditorModel->RebuildEditorMaskInfo();
		}
	}

}	// namespace UE::NeuralMorphModel

#undef LOCTEXT_NAMESPACE
