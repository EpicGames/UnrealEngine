// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerMorphModelEditorModel.h"
#include "NeuralMorphModel.h"

#define UE_API NEURALMORPHMODELEDITOR_API

class UNeuralMorphModel;

namespace UE::MLDeformer
{
	class SMLDeformerInputWidget;
}

namespace UE::NeuralMorphModel
{
	using namespace UE::MLDeformer;

	/**
	 * The editor model related to the neural morph model's runtime class (UNeuralMorphModel).
	 */
	class FNeuralMorphEditorModel
		: public FMLDeformerMorphModelEditorModel
	{
	public:
		// We need to implement this static MakeInstance method.
		static UE_API FMLDeformerEditorModel* MakeInstance();

		// FGCObject overrides.
		virtual FString GetReferencerName() const override		{ return TEXT("FNeuralMorphEditorModel"); }
		// ~END FGCObject overrides.
	
		// FMLDeformerEditorModel overrides.
		UE_API virtual void Init(const InitSettings& Settings) override;
		UE_API virtual ETrainingResult Train() override;
		UE_API virtual void InitInputInfo(UMLDeformerInputInfo* InputInfo) override;
		UE_API virtual FText GetOverlayText() const override;
		UE_API virtual void OnPropertyChanged(FPropertyChangedEvent& PropertyChangedEvent) override;
		UE_API virtual TSharedPtr<SMLDeformerInputWidget> CreateInputWidget() override;
		UE_API virtual void OnPostInputAssetChanged() override;
		UE_API virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
		virtual bool GetSupportsPerTrainingInputAnimVertexMask() const override	{ return true; }
		UE_API virtual void UpdateTrainingDeviceList() override;
		UE_API virtual UMeshDeformer* GetActiveHeatMapDeformer() const override;
		UE_API virtual void UpdateIsReadyForTrainingState() override;
		UE_API virtual bool LoadTrainedNetwork() const override;
		// ~END FMLDeformerEditorModel overrides.

		// FMLDeformerMorphModelEditorModel overrides.
		UE_API virtual const TArrayView<const float> GetMaskForMorphTarget(int32 MorphTargetIndex) const override;
		virtual bool IsInputMaskingSupported() const override	{ return true; }
		// ~FMLDeformerMorphModelEditorModel overrides.

		/** Get a pointer to the neural morph runtime model. */
		UNeuralMorphModel* GetNeuralMorphModel() const			{ return Cast<UNeuralMorphModel>(Model); }

		/**
		 * Build a set of masks, all concatenated into one flat array.
		 * The number of masks is (NumBones + NumCurves + NumBoneGroups + NumCurveGroups).
		 * Each mask contains Model->GetNumBaseMeshVerts() number of floats.
		 * The masks for curves and curve groups are all set to 1.0.
		 * The masks for bones and bone groups contain all the vertices they influence during skinning, as well as the ones from the parent bone.
		 * This will modify the item mask buffer as returned by UMLDeformerMorphModel::GetInputItemMaskBuffer().
		 * @param OutMaskBuffer The mask buffer to write to. This will automatically be resized.
		 */
		UE_API void BuildMaskBuffer(TArray<float>& OutMaskBuffer);

		/** Rebuild the mask buffer. */
		UE_API void RebuildEditorMaskInfo();

		/**
		 * Generate a unique name for a bone group.
		 * @return The unique name.
		 */
		UE_API FName GenerateUniqueBoneGroupName() const;

		/**
		 * Generate a unique name for a curve group.
		 * @return The unique name.
		 */
		UE_API FName GenerateUniqueCurveGroupName() const;

		/**
		 * Remove mask infos for bones that do not exist in the bone include list.
		 */
		UE_API void RemoveNonExistingMaskInfos();

		/** Clear all bone mask infos. */
		UE_API void ResetBoneMaskInfos();

		/** Clear all bone group mask infos. */
		UE_API void ResetBoneGroupMaskInfos();

		/**
		 * Generate bone mask infos for all bones in the bone input list.
		 * Use a specific hierarchy depth for this, which defines how deep up and down the hierarchy we should traverse for each specific bone.
		 * Imagine we generate the mask for a bone in a chain of bones. We look at all vertices skinned to this bone, and include those vertices in the mask.
		 * With a hierarchy depth of 1, include all vertices skinned to its parent bone as well as to its child bone.
		 * With a hierarchy depth of 2 we also include the parent of the parent and the child of the child if it exists.
		 * We always want the mask to be large enough, otherwise if say skin stretches a lot, we will not capture the vertex deltas for those vertices far away from the bone, which can cause issues.
		 * However we don't want the mask to be too large either, as that costs performance and memory usage.
		 * @param HierarchyDepth The hierarchy depth to use, which should be a value of 1 or above.
		 */
		UE_API void GenerateBoneMaskInfos(int32 HierarchyDepth);

		/**
		 * Generate the mask infos for all bone groups. 
		 * Look at the GenerateBoneMaskInfos method for a detailed description of the HierarchyDepth.
		 * @param HierarchyDepth The hierarchy depth to use, which should be a value of 1 or above.
		 * @see GenerateBoneMaskInfos
		 */
		UE_API void GenerateBoneGroupMaskInfos(int32 HierarchyDepth);

		/**
		 * Generate the mask info for a given bone.
		 * Look at the GenerateBoneMaskInfos method for a detailed description of the HierarchyDepth.
		 * @param InputInfoBoneIndex The bone index inside the input info object.
		 * @param HierarchyDepth The hierarchy depth to use, which should be a value of 1 or above.
		 * @see GenerateBoneMaskInfos
		 */
		UE_API void GenerateBoneMaskInfo(int32 InputInfoBoneIndex, int32 HierarchyDepth);

		/**
		 * Generate the mask info for a given bone group.
		 * Look at the GenerateBoneMaskInfos method for a detailed description of the HierarchyDepth.
		 * @param InputInfoBoneIndex The bone index inside the input info object.
		 * @param HierarchyDepth The hierarchy depth to use, which should be a value of 1 or above.
		 * @see GenerateBoneMaskInfos
		 */
		UE_API void GenerateBoneGroupMaskInfo(int32 InputInfoBoneGroupIndex, int32 HierarchyDepth);

		/**
		 * Set the mask visualization item index. This specifies which item (bone, curve, bone group, curve group) to visualize the mask for.
		 * The first index values are for all input bones. Indices that come after that are for curves, then for bone groups and finally followed by curve groups.
		 * So if there are 10 input bones and 4 bone groups, and the visualization index is 11, it means it's for the second bone group.
		 * @param InMaskVizItemIndex The mask visualization index.
		 */
		void SetMaskVisualizationItemIndex(int32 InMaskVizItemIndex)	{ MaskVizItemIndex = InMaskVizItemIndex; }

		/**
		 * Get the mask visualization item index. This specifies which item (bone, curve, bone group, curve group) to visualize the mask for.
		 * The first index values are for all input bones. Indices that come after that are for curves, then for bone groups and finally followed by curve groups.
		 * So if there are 10 input bones and 4 bone groups, and the visualization index is 11, it means it's for the second bone group.
		 * @return The item to visualize the mask for.
		 */
		int32 GetMaskVisualizationItemIndex() const						{ return MaskVizItemIndex; }

		/**
		 * Draw the mask for a given item. An item can be a bone, curve or anything else that has a mask.
		 * @param PDI The debug draw interface.
		 * @param MaskItemIndex The item index. The valid range on default is [0..NumEditorInfoBones + NumEditorInfoCurves - 1]. The curves are attached after the bones.
		 *        So if you have 10 bones, and 5 curves, and the MaskItemIndex is value 12, it will draw the mask for the third curve, because items [0..9] would be for the bones, and [10..14] for the curves.
		 * @param DrawOffset The offset in world space units to draw the mask at.
		 */
		UE_API void DebugDrawItemMask(FPrimitiveDrawInterface* PDI, int32 MaskItemIndex, const FVector& DrawOffset);

	private:
		/**
		 * Add child twist bones to the list of skeletal bone indices.
		 * This will iterate over all bones inside the SkelBoneIndices array and check if there are direct child nodes inside the ref skeleton that contain
		 * the substring "twist" inside their name. If so, that bone will be added to the SkelBoneIndices array as well.
		 * The "twist" string is case-insensitive. The "twist" string can be configured on a per project basis inside the UNeuralMorphModel::TwistBoneSubString property, which 
		 * is exposed in the per project .ini file. On default this value is "twist".
		 * @param RefSkel The reference skeleton we use to find child nodes.
		 * @param SkelBoneIndices The input and output array of indices inside the reference skeleton. This method potentially adds new entries to this array.
		 */
		UE_API void AddTwistBones(const FReferenceSkeleton& RefSkel, TArray<int32>& SkelBoneIndices);

	protected:
		int32 MaskVizItemIndex = INDEX_NONE;
	};
}	// namespace UE::NeuralMorphModel

#undef UE_API
