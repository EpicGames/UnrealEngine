// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Engine/DeveloperSettings.h"
#include "NeuralMorphEditorProjectSettings.generated.h"

#define UE_API NEURALMORPHMODELEDITOR_API


UCLASS(MinimalAPI, config = EditorPerProjectUserSettings)
class UNeuralMorphEditorProjectSettings
	: public UDeveloperSettings
{
	GENERATED_BODY()

public:
	static UE_API const UNeuralMorphEditorProjectSettings* Get();

	// UDeveloperSettings overrides
	virtual FName GetContainerName() const	{ return FName("Project"); }
	virtual FName GetCategoryName() const	{ return FName("Plugins"); }
	virtual FName GetSectionName() const	{ return FName("Neural Morph Model"); }
#if WITH_EDITOR
	UE_API virtual FText GetSectionText() const override;
	UE_API virtual FText GetSectionDescription() const override;
#endif
	// ~END UDeveloperSettings overrides

	/** 
	 * The bone name filter that identifies twist bones.
	 * Any bone name with this string inside its name will be seen as a twist bone.
	 * This is only used during bone and bone group mask generation inside the Neural Morph Model.
	 * Twist bones that are a child of the bones included in the bone mask are automatically included inside the mask generation as well.
	 * We use this case-insensitive sub-string to identify twist bones. Set this to an empty string to disable automatically adding twist bones.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Bone and Bone Group masking")
	FString TwistBoneFilter = TEXT("twist");
};

#undef UE_API
