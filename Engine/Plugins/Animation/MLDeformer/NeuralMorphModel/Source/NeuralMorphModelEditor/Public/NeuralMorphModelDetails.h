// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerMorphModelDetails.h"
#include "IDetailCustomization.h"

#define UE_API NEURALMORPHMODELEDITOR_API

class IDetailLayoutBuilder;

namespace UE::NeuralMorphModel
{
	/**
	 * The detail customization for the neural morph model.
	 */
	class FNeuralMorphModelDetails
		: public UE::MLDeformer::FMLDeformerMorphModelDetails
	{
	public:
		/** Makes a new instance of this detail layout class for a specific detail view requesting it. */
		static UE_API TSharedRef<IDetailCustomization> MakeInstance();

		// ILayoutDetails overrides.
		UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
		// ~END ILayoutDetails overrides.
	};
}	// namespace UE::NeuralMorphModel

#undef UE_API
