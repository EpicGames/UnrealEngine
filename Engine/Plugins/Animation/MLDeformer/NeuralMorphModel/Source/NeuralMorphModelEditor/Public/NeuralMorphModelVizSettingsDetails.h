// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerMorphModelVizSettingsDetails.h"

namespace UE::NeuralMorphModel
{
	/**
	 * The neural morph model visualization settings detail customization.
	 * This is inherited from the Morph Model detail customization.
	 * We do not have to do anything else. We still need this class in order to do our
	 * detail customization correctly. Just using the Morph Model directly would cause issues.
	 */
	class FNeuralMorphModelVizSettingsDetails
		: public UE::MLDeformer::FMLDeformerMorphModelVizSettingsDetails
	{
	public:
		static TSharedRef<IDetailCustomization> MakeInstance()
		{
			return MakeShareable(new FNeuralMorphModelVizSettingsDetails());
		}
	};
}	// namespace UE::NeuralMorphModel
