// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "MLDeformerGeomCacheTrainingModel.h"
#include "NeuralMorphTrainingModel.generated.h"

#define UE_API NEURALMORPHMODELEDITOR_API

/**
 * The training model for the neural morph model.
 * This class is our link to the Python training.
 */
UCLASS(MinimalAPI, Blueprintable)
class UNeuralMorphTrainingModel
	: public UMLDeformerGeomCacheTrainingModel
{
	GENERATED_BODY()

public:
	UE_API virtual void Init(UE::MLDeformer::FMLDeformerEditorModel* InEditorModel) override;

	/**
	 * Main training function, with implementation in python.
	 * You need to implement this method. See the UMLDeformerTrainingModel documentation for more details.
	 * @see UMLDeformerTrainingModel
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = "Training Model")
	UE_API int32 Train() const;

	UFUNCTION(BlueprintPure, Category = "Training Model")
	UE_API int32 GetNumBoneGroups() const;

	UFUNCTION(BlueprintPure, Category = "Training Model")
	UE_API int32 GetNumCurveGroups() const;

	UFUNCTION(BlueprintPure, Category = "Training Model")
	UE_API TArray<int32> GenerateBoneGroupIndices() const;

	UFUNCTION(BlueprintPure, Category = "Training Model")
	UE_API TArray<int32> GenerateCurveGroupIndices() const;

	UFUNCTION(BlueprintPure, Category = "Training Model")
	UE_API TArray<float> GetMorphTargetMasks() const;
};

#undef UE_API
