// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"
#include "Framework/Commands/Commands.h"

#define UE_API NEURALMORPHMODELEDITOR_API

class SWidget;

namespace UE::MLDeformer
{
	class SMLDeformerInputWidget;
}

namespace UE::NeuralMorphModel
{
	class FNeuralMorphEditorModel;
	class SNeuralMorphBoneGroupsWidget;
	class SNeuralMorphInputWidget;
	class FNeuralMorphBoneGroupsCommands
		: public TCommands<FNeuralMorphBoneGroupsCommands>
	{
	public:
		FNeuralMorphBoneGroupsCommands();
		virtual void RegisterCommands() override;

	public:
		TSharedPtr<FUICommandInfo> CreateGroup;
		TSharedPtr<FUICommandInfo> DeleteSelectedItems;
		TSharedPtr<FUICommandInfo> ClearGroups;
		TSharedPtr<FUICommandInfo> AddBoneToGroup;
		TSharedPtr<FUICommandInfo> CopyBoneGroups;
		TSharedPtr<FUICommandInfo> PasteBoneGroups;
	};


	class FNeuralMorphBoneGroupsTreeElement
		: public TSharedFromThis<FNeuralMorphBoneGroupsTreeElement>
	{
	public:
		TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FNeuralMorphBoneGroupsTreeElement> InTreeElement, TSharedPtr<SNeuralMorphBoneGroupsWidget> InTreeWidget);

		bool IsGroup() const { return GroupIndex != INDEX_NONE; }

	public:
		FName Name;
		TArray<TSharedPtr<FNeuralMorphBoneGroupsTreeElement>> Children;
		TWeakPtr<FNeuralMorphBoneGroupsTreeElement> ParentGroup;
		FSlateColor TextColor;
		int32 GroupIndex = INDEX_NONE;
		int32 GroupBoneIndex = INDEX_NONE;
	};


	class SNeuralMorphBoneGroupsTreeRowWidget 
		: public STableRow<TSharedPtr<FNeuralMorphBoneGroupsTreeElement>>
	{
	public:
		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, TSharedRef<FNeuralMorphBoneGroupsTreeElement> InTreeElement, TSharedPtr<SNeuralMorphBoneGroupsWidget> InTreeView);

	private:
		TWeakPtr<FNeuralMorphBoneGroupsTreeElement> WeakTreeElement;
		FText GetName() const;

		friend class SNeuralMorphBoneGroupsWidget; 
	};


	class SNeuralMorphBoneGroupsWidget
		: public STreeView<TSharedPtr<FNeuralMorphBoneGroupsTreeElement>>
	{
	public:
		SLATE_BEGIN_ARGS(SNeuralMorphBoneGroupsWidget) {}
		SLATE_ARGUMENT(FNeuralMorphEditorModel*, EditorModel)
		SLATE_ARGUMENT(TSharedPtr<UE::MLDeformer::SMLDeformerInputWidget>, InputWidget)
		SLATE_END_ARGS()

		UE_API void Construct(const FArguments& InArgs);
		UE_API void BindCommands(TSharedPtr<FUICommandList> CommandList);
		UE_API void Refresh(bool bBroadcastPropertyChanged=false);

		UE_API FText GetSectionTitle() const;
		UE_API int32 GetNumSelectedGroups() const;
		UE_API TSharedPtr<UE::MLDeformer::SMLDeformerInputWidget> GetInputWidget() const;

	private:
		UE_API void AddElement(TSharedPtr<FNeuralMorphBoneGroupsTreeElement> Element, TSharedPtr<FNeuralMorphBoneGroupsTreeElement> ParentElement);	
		UE_API void HandleGetChildrenForTree(TSharedPtr<FNeuralMorphBoneGroupsTreeElement> InItem, TArray<TSharedPtr<FNeuralMorphBoneGroupsTreeElement>>& OutChildren);	
		UE_API void OnSelectionChanged(TSharedPtr<FNeuralMorphBoneGroupsTreeElement> Selection, ESelectInfo::Type SelectInfo);
		UE_API const TArray<TSharedPtr<FNeuralMorphBoneGroupsTreeElement>>& GetRootElements() const;
		UE_API virtual FReply OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
		UE_API TSharedRef<ITableRow> MakeTableRowWidget(TSharedPtr<FNeuralMorphBoneGroupsTreeElement> InItem, const TSharedRef<STableViewBase>& OwnerTable);
		UE_API TSharedPtr<SWidget> CreateContextMenuWidget() const;

		UE_API void UpdateTreeElements();
		UE_API void RefreshTree(bool bBroadcastPropertyChanged);
		UE_API bool BroadcastModelPropertyChanged(const FName PropertyName);
		UE_API TSharedPtr<SWidget> CreateContextWidget() const;

		UE_API void OnCopyBoneGroups() const;
		UE_API void OnPasteBoneGroups();
	
		UE_API void OnCreateBoneGroup();
		UE_API void OnDeleteSelectedItems();
		UE_API void OnClearBoneGroups();
		UE_API void OnAddBoneToGroup();

	private:
		TArray<TSharedPtr<FNeuralMorphBoneGroupsTreeElement>> RootElements;
		TSharedPtr<UE::MLDeformer::SMLDeformerInputWidget> InputWidget;
		FNeuralMorphEditorModel* EditorModel = nullptr;
		FText SectionTitle;
	};

}	// namespace UE::NeuralMorphModel

#undef UE_API
