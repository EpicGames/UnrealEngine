// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"
#include "Framework/Commands/Commands.h"
#include "MLDeformerEditorStyle.h"

#define UE_API NEURALMORPHMODELEDITOR_API

class SWidget;

namespace UE::MLDeformer
{
	class SMLDeformerInputWidget;
};

namespace UE::NeuralMorphModel
{
	class FNeuralMorphEditorModel;
	class SNeuralMorphCurveGroupsWidget;

	class FNeuralMorphCurveGroupsCommands
		: public TCommands<FNeuralMorphCurveGroupsCommands>
	{
	public:
		FNeuralMorphCurveGroupsCommands();
		virtual void RegisterCommands() override;

	public:
		TSharedPtr<FUICommandInfo> CreateGroup;
		TSharedPtr<FUICommandInfo> DeleteSelectedItems;
		TSharedPtr<FUICommandInfo> ClearGroups;
		TSharedPtr<FUICommandInfo> AddCurveToGroup;
		TSharedPtr<FUICommandInfo> CopyCurveGroups;
		TSharedPtr<FUICommandInfo> PasteCurveGroups;
	};


	class FNeuralMorphCurveGroupsTreeElement
		: public TSharedFromThis<FNeuralMorphCurveGroupsTreeElement>
	{
	public:
		TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FNeuralMorphCurveGroupsTreeElement> InTreeElement, TSharedPtr<SNeuralMorphCurveGroupsWidget> InTreeWidget);

		bool IsGroup() const { return GroupIndex != INDEX_NONE; }

	public:
		FName Name;
		TArray<TSharedPtr<FNeuralMorphCurveGroupsTreeElement>> Children;
		TWeakPtr<FNeuralMorphCurveGroupsTreeElement> ParentGroup;
		FSlateColor TextColor;
		int32 GroupIndex = INDEX_NONE;
		int32 GroupCurveIndex = INDEX_NONE;
	};


	class SNeuralMorphCurveGroupsTreeRowWidget 
		: public STableRow<TSharedPtr<FNeuralMorphCurveGroupsTreeElement>>
	{
	public:
		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, TSharedRef<FNeuralMorphCurveGroupsTreeElement> InTreeElement, TSharedPtr<SNeuralMorphCurveGroupsWidget> InTreeView);

	private:
		TWeakPtr<FNeuralMorphCurveGroupsTreeElement> WeakTreeElement;
		FText GetName() const;

		friend class SNeuralMorphCurveGroupsWidget; 
	};


	class SNeuralMorphCurveGroupsWidget
		: public STreeView<TSharedPtr<FNeuralMorphCurveGroupsTreeElement>>
	{
	public:
		SLATE_BEGIN_ARGS(SNeuralMorphCurveGroupsWidget) {}
		SLATE_ARGUMENT(FNeuralMorphEditorModel*, EditorModel)
		SLATE_ARGUMENT(TSharedPtr<UE::MLDeformer::SMLDeformerInputWidget>, InputWidget)
		SLATE_END_ARGS()

		UE_API void Construct(const FArguments& InArgs);
		UE_API void BindCommands(TSharedPtr<FUICommandList> CommandList);
		UE_API void Refresh(bool bBroadcastPropertyChanged=false);

		UE_API FText GetSectionTitle() const;
		UE_API int32 GetNumSelectedGroups() const;
		UE_API TSharedPtr<UE::MLDeformer::SMLDeformerInputWidget> GetInputWidget() const;

	private:
		UE_API void AddElement(TSharedPtr<FNeuralMorphCurveGroupsTreeElement> Element, TSharedPtr<FNeuralMorphCurveGroupsTreeElement> ParentElement);	
		UE_API void HandleGetChildrenForTree(TSharedPtr<FNeuralMorphCurveGroupsTreeElement> InItem, TArray<TSharedPtr<FNeuralMorphCurveGroupsTreeElement>>& OutChildren);	
		UE_API void OnSelectionChanged(TSharedPtr<FNeuralMorphCurveGroupsTreeElement> Selection, ESelectInfo::Type SelectInfo);
		UE_API const TArray<TSharedPtr<FNeuralMorphCurveGroupsTreeElement>>& GetRootElements() const;
		UE_API virtual FReply OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
		UE_API TSharedRef<ITableRow> MakeTableRowWidget(TSharedPtr<FNeuralMorphCurveGroupsTreeElement> InItem, const TSharedRef<STableViewBase>& OwnerTable);
		UE_API TSharedPtr<SWidget> CreateContextMenuWidget() const;

		UE_API void UpdateTreeElements();
		UE_API void RefreshTree(bool bBroadcastPropertyChanged);
		UE_API bool BroadcastModelPropertyChanged(const FName PropertyName);
		UE_API TSharedPtr<SWidget> CreateContextWidget() const;

		UE_API void OnCopyCurveGroups() const;
		UE_API void OnPasteCurveGroups();
		
		UE_API void OnCreateCurveGroup();
		UE_API void OnDeleteSelectedItems();
		UE_API void OnClearCurveGroups();
		UE_API void OnAddCurveToGroup();

	private:
		TArray<TSharedPtr<FNeuralMorphCurveGroupsTreeElement>> RootElements;
		TSharedPtr<UE::MLDeformer::SMLDeformerInputWidget> InputWidget;
		FNeuralMorphEditorModel* EditorModel = nullptr;
		FText SectionTitle;
	};

}	// namespace UE::NeuralMorphModel

#undef UE_API
