// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "SMLDeformerInputWidget.h"
#include "MeshDescription.h"
#include "Framework/Commands/Commands.h"

#define UE_API NEURALMORPHMODELEDITOR_API

class FMenuBuilder;
class USkeletalMesh;
struct FMLDeformerMaskInfo;

namespace UE::MLDeformer
{
	class FMLDeformerEditorModel;
};

namespace UE::NeuralMorphModel
{
	class SNeuralMorphBoneGroupsWidget;
	class SNeuralMorphCurveGroupsWidget;
	class FNeuralMorphBoneGroupsTreeElement;
	class FNeuralMorphCurveGroupsTreeElement;

	class NEURALMORPHMODELEDITOR_API FNeuralMorphInputWidgetCommands
		: public TCommands<FNeuralMorphInputWidgetCommands>
	{
	public:
		FNeuralMorphInputWidgetCommands();
		virtual void RegisterCommands() override;

	public:
		TSharedPtr<FUICommandInfo> ResetAllBoneMasks;
		TSharedPtr<FUICommandInfo> ResetSelectedBoneMasks;
		TSharedPtr<FUICommandInfo> ExpandSelectedBoneMasks;

		TSharedPtr<FUICommandInfo> ResetAllBoneGroupMasks;
		TSharedPtr<FUICommandInfo> ResetSelectedBoneGroupMasks;
		TSharedPtr<FUICommandInfo> ExpandSelectedBoneGroupMasks;

		TSharedPtr<FUICommandInfo> ConfigureBoneMask;
		TSharedPtr<FUICommandInfo> ConfigureBoneGroupMask;
	};


	class SNeuralMorphInputWidget
		: public UE::MLDeformer::SMLDeformerInputWidget
	{
	public:
		SLATE_BEGIN_ARGS(SNeuralMorphInputWidget) {}
		SLATE_ARGUMENT(UE::MLDeformer::FMLDeformerEditorModel*, EditorModel)
		SLATE_END_ARGS()

		UE_API void Construct(const FArguments& InArgs);

		// SMLDeformerInputWidget overrides.
		UE_API virtual void Refresh() override;
		UE_API virtual void AddInputBonesMenuItems(FMenuBuilder& MenuBuilder) override;
		UE_API virtual void AddInputBonesPlusIconMenuItems(FMenuBuilder& MenuBuilder) override;
		UE_API virtual void OnAddInputBones(const TArray<FName>& Names) override;
		UE_API virtual void OnAddInputCurves(const TArray<FName>& Names) override;
		UE_API virtual void OnAddAnimatedBones() override;
		UE_API virtual void OnClearInputBones() override;
		UE_API virtual void OnDeleteInputBones(const TArray<FName>& Names) override;
		UE_API virtual void OnDeleteInputCurves(const TArray<FName>& Names) override;
		UE_API virtual void OnSelectInputBone(FName BoneName) override;
		UE_API virtual void OnSelectInputCurve(FName BoneName) override;
		UE_API virtual void OnAddAnimatedCurves() override;
		UE_API virtual void ClearSelectionForAllWidgetsExceptThis(TSharedPtr<SWidget> ExceptThisWidget) override;
		UE_API virtual TSharedPtr<SWidget> GetExtraBonePickerWidget() override;
		// ~END SMLDeformerInputWidget overrides.

		UE_API void AddInputBoneGroupsMenuItems(FMenuBuilder& MenuBuilder);
		UE_API void AddInputBoneGroupsPlusIconMenuItems(FMenuBuilder& MenuBuilder);

		UE_API void OnSelectInputBoneGroup(TSharedPtr<FNeuralMorphBoneGroupsTreeElement> Element);
		UE_API void OnSelectInputCurveGroup(TSharedPtr<FNeuralMorphCurveGroupsTreeElement> Element);

		TSharedPtr<FUICommandList> GetBoneGroupsCommandList() const	{ return BoneGroupsCommandList; }
		TSharedPtr<FUICommandList> GetCurveGroupsCommandList() const { return CurveGroupsCommandList; }

		int32 GetHierarchyDepth() const { return HierarchyDepth; }

	protected:
		UE_API void CreateBoneGroupsSection();
		UE_API void CreateCurveGroupsSection();

		UE_API void ResetAllBoneMasks();
		UE_API void ResetSelectedBoneMasks();
		UE_API void ExpandBoneMasks();

		UE_API void ResetAllBoneGroupMasks();
		UE_API void ResetSelectedBoneGroupMasks();
		UE_API void ExpandBoneGroupMasks();

		UE_API void ConfigureBoneMask();
		UE_API void ConfigureBoneGroupMask();

		UE_API void BindCommands();
		UE_API FReply ShowBoneGroupsManageContextMenu();
		UE_API FReply ShowCurveGroupsManageContextMenu();

	protected:
		TSharedPtr<SNeuralMorphBoneGroupsWidget> BoneGroupsWidget;
		TSharedPtr<SNeuralMorphCurveGroupsWidget> CurveGroupsWidget;
		TSharedPtr<FUICommandList> BoneGroupsCommandList;
		TSharedPtr<FUICommandList> CurveGroupsCommandList;
		int32 HierarchyDepth = 2;
	};
}	// namespace UE::NeuralMorphModel

#undef UE_API
